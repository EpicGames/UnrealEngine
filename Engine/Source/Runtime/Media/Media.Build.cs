// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class Media : ModuleRules
	{
		public Media(ReadOnlyTargetRules Target) : base(Target)
		{
			IWYUSupport = IWYUSupport.KeepAsIsForNow;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"RenderCore",
				});

			PublicIncludePathModuleNames.AddRange(
				new string[] {
					"RenderCore",
				});
		}
	}
}
