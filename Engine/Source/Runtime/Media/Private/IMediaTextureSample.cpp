// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMediaTextureSample.h"

namespace MediaTextureSampleFormat
{
	const TCHAR* EnumToString(const EMediaTextureSampleFormat InSampleFormat)
	{
		switch(InSampleFormat)
		{
			case EMediaTextureSampleFormat::CharAYUV: return TEXT("CharAYUV");
			case EMediaTextureSampleFormat::CharBGRA: return TEXT("CharBGRA");
			case EMediaTextureSampleFormat::CharBGR10A2: return TEXT("CharBGR10A2");
			case EMediaTextureSampleFormat::CharBMP: return TEXT("CharBMP");
			case EMediaTextureSampleFormat::CharNV12: return TEXT("CharNV12");
			case EMediaTextureSampleFormat::CharNV21: return TEXT("CharNV21");
			case EMediaTextureSampleFormat::CharUYVY: return TEXT("CharUYVY");
			case EMediaTextureSampleFormat::CharYUY2: return TEXT("CharYUY2");
			case EMediaTextureSampleFormat::CharYVYU: return TEXT("CharYVYU");
			case EMediaTextureSampleFormat::FloatRGB: return TEXT("FloatRGB");
			case EMediaTextureSampleFormat::FloatRGBA: return TEXT("FloatRGBA");
			case EMediaTextureSampleFormat::Y416: return TEXT("Y416");
			case EMediaTextureSampleFormat::P010: return TEXT("P010");
			case EMediaTextureSampleFormat::DXT1: return TEXT("DXT1");
			case EMediaTextureSampleFormat::DXT5: return TEXT("DXT5");
			case EMediaTextureSampleFormat::YCoCg_DXT5: return TEXT("YCoCg_DXT5");
			case EMediaTextureSampleFormat::YCoCg_DXT5_Alpha_BC4: return TEXT("YCoCg_DXT5_Alpha_BC4");
			case EMediaTextureSampleFormat::Undefined:
			default: return TEXT("Undefined");
		}
	}
}
