// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaClock.h"

#include "CoreTypes.h"

#include "IMediaClockSink.h"
#include "Misc/ScopeLock.h"

/* FMediaClock structors
 *****************************************************************************/

FMediaClock::FMediaClock()
	: DeltaTime(FTimespan::Zero())
	, Timecode(FTimespan::MinValue())
{ }


/* FMediaClock interface
 *****************************************************************************/

void FMediaClock::TickFetch()
{
	UpdateSinkArray();

	for (int32 SinkIndex = Sinks.Num() - 1; SinkIndex >= 0; --SinkIndex)
	{
		auto Sink = Sinks[SinkIndex].Pin();

		if (Sink.IsValid())
		{
			Sink->TickFetch(DeltaTime, Timecode);
		}
		else
		{
			Sinks.RemoveAtSwap(SinkIndex);
		}
	}
}


void FMediaClock::TickInput()
{
	UpdateSinkArray();

	for (int32 SinkIndex = Sinks.Num() - 1; SinkIndex >= 0; --SinkIndex)
	{
		auto Sink = Sinks[SinkIndex].Pin();

		if (Sink.IsValid())
		{
			Sink->TickInput(DeltaTime, Timecode);
		}
		else
		{
			Sinks.RemoveAtSwap(SinkIndex);
		}
	}
}


void FMediaClock::TickOutput()
{
	UpdateSinkArray();

	for (int32 SinkIndex = Sinks.Num() - 1; SinkIndex >= 0; --SinkIndex)
	{
		auto Sink = Sinks[SinkIndex].Pin();

		if (Sink.IsValid())
		{
			Sink->TickOutput(DeltaTime, Timecode);
		}
		else
		{
			Sinks.RemoveAtSwap(SinkIndex);
		}
	}
}


void FMediaClock::TickRender()
{
	UpdateSinkArray();

	for (int32 SinkIndex = Sinks.Num() - 1; SinkIndex >= 0; --SinkIndex)
	{
		auto Sink = Sinks[SinkIndex].Pin();

		if (Sink.IsValid())
		{
			Sink->TickRender(DeltaTime, Timecode);
		}
		else
		{
			Sinks.RemoveAtSwap(SinkIndex);
		}
	}
}


void FMediaClock::UpdateTimecode(const FTimespan NewTimecode, bool NewTimecodeLocked)
{
	if (Timecode == FTimespan::MinValue())
	{
		DeltaTime = FTimespan::Zero();
	}
	else
	{
		DeltaTime = NewTimecode - Timecode;
	}

	Timecode = NewTimecode;
	TimecodeLocked = NewTimecodeLocked;
}


/* IMediaClock interface
 *****************************************************************************/

void FMediaClock::AddSink(const TSharedRef<IMediaClockSink, ESPMode::ThreadSafe>& Sink)
{
	FScopeLock Lock(&SinkCriticalSection);

	SinksToAdd.AddUnique(Sink);

	// Just in case this sink is pending to be removed.
	if (SinksToRemove.Contains(Sink))
	{
		SinksToRemove.RemoveSwap(Sink);
	}
}


FTimespan FMediaClock::GetTimecode() const
{
	return Timecode;
}


bool FMediaClock::IsTimecodeLocked() const
{
	return TimecodeLocked;
}


void FMediaClock::RemoveSink(const TSharedRef<IMediaClockSink, ESPMode::ThreadSafe>& Sink)
{
	FScopeLock Lock(&SinkCriticalSection);

	SinksToRemove.AddUnique(Sink);

	// Just in case this sink is pending to be added.
	if (SinksToAdd.Contains(Sink))
	{
		SinksToAdd.RemoveSwap(Sink);
	}
}

void FMediaClock::UpdateSinkArray()
{
	FScopeLock Lock(&SinkCriticalSection);

	// Add new sinks.
	for (auto Sink : SinksToAdd)
	{
		Sinks.AddUnique(Sink);
	}
	SinksToAdd.Empty();
	
	// Remove sinks.
	for (auto Sink : SinksToRemove)
	{
		Sinks.RemoveSwap(Sink);
	}
	SinksToRemove.Empty();
}
