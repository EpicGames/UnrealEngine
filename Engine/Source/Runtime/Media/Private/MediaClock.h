// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"

#include "IMediaClock.h"

class IMediaClockSink;


/**
 * Implements the media clock.
 */
class FMediaClock
	: public IMediaClock
{
public:

	/** Default constructor. */
	FMediaClock();

public:

	/**
	 * Tick all clock sinks in the Input stage.
	 *
	 * @see TickOutput, TickRender, TickUpdate
	 */
	void TickInput();

	/**
	 * Tick all clock sinks in the Output stage.
	 *
	 * @see TickInput, TickRender, TickUpdate
	 */
	void TickOutput();

	/**
	 * Tick all clock sinks in the Render stage.
	 *
	 * @see TickInput, TickOutput, TickUpdate
	 */
	void TickRender();

	/**
	 * Tick all clock sinks in the Update stage.
	 *
	 * @see TickInput, TickOutput, TickRender
	 */
	void TickFetch();

	/**
	 * Update the current time code.
	 *
	 * @param NewTimecode The new time code.
	 * @param NewTimecodeLocked Whether media objects should lock to the time code.
	 */
	void UpdateTimecode(const FTimespan NewTimecode, bool NewTimecodeLocked);

public:

	//~ IMediaClock interface

	virtual void AddSink(const TSharedRef<IMediaClockSink, ESPMode::ThreadSafe>& Sink) override;
	virtual FTimespan GetTimecode() const override;
	virtual bool IsTimecodeLocked() const override;
	virtual void RemoveSink(const TSharedRef<IMediaClockSink, ESPMode::ThreadSafe>& Sink) override;

private:

	/** The current delta time. */
	FTimespan DeltaTime;

	/** Whether media objects should lock to the media clock's time code. */
	bool Locked;

	/** Registered clock sinks. */
	TArray<TWeakPtr<IMediaClockSink, ESPMode::ThreadSafe>> Sinks;

	/** Sinks that need to be added. */
	TArray<TWeakPtr<IMediaClockSink, ESPMode::ThreadSafe>> SinksToAdd;

	/** Sinks that need to be removed. */
	TArray<TWeakPtr<IMediaClockSink, ESPMode::ThreadSafe>> SinksToRemove;

	/** Critical section for synchronizing access to sink. */
	mutable FCriticalSection SinkCriticalSection;

	/** The current time code. */
	FTimespan Timecode;

	/** Whether the time code is locked to an external clock. */
	bool TimecodeLocked;

	/**
	 * Updates the Sink array to reflect what has been added and removed since the last call to UpdateSinkArray.
	 */
	void UpdateSinkArray();
	
};
