// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMediaModule.h"

#include "CoreMinimal.h"
#include "HAL/RunnableThread.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "Stats/Stats.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"

#include "IMediaCaptureSupport.h"
#include "IMediaPlayerFactory.h"
#include "IMediaTimeSource.h"
#include "MediaClock.h"
#include "MediaTicker.h"

CSV_DEFINE_CATEGORY_MODULE(MEDIA_API, MediaStreaming, false);

// ------------------------------------------------------------------------------------------------------------------------------

/**
 * Implements the Media module.
 */
class FMediaModule
	: public IMediaModule
{

public:

	//~ IMediaModule interface

	virtual FName GetPlatformName(const FGuid& PlatformGuid) const override
	{
		for (auto& Pair : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
		{
			if (Pair.Value.GlobalIdentifier == PlatformGuid)
			{
				return Pair.Key;
			}
		}
		return FName();
	}

	virtual FGuid GetPlatformGuid(const FName& PlatformName) const override
	{
		return FDataDrivenPlatformInfoRegistry::GetPlatformInfo(PlatformName).GlobalIdentifier;
	}

	virtual const TArray<IMediaCaptureSupport*>& GetCaptureSupports() const override
	{
		return CaptureSupports;
	}

	virtual IMediaClock& GetClock() override
	{
		return Clock;
	}

	virtual const TArray<IMediaPlayerFactory*>& GetPlayerFactories() const override
	{
		return PlayerFactories;
	}

	virtual IMediaPlayerFactory* GetPlayerFactory(const FName& FactoryName) const override
	{
		for (IMediaPlayerFactory* Factory : PlayerFactories)
		{
			if (Factory->GetPlayerName() == FactoryName)
			{
				return Factory;
			}
		}

		return nullptr;
	}

	virtual IMediaPlayerFactory* GetPlayerFactory(const FGuid& PlayerpPluginGUID) const override
	{
		for (IMediaPlayerFactory* Factory : PlayerFactories)
		{
			if (Factory->GetPlayerPluginGUID() == PlayerpPluginGUID)
			{
				return Factory;
			}
		}

		return nullptr;
	}

	virtual IMediaTicker& GetTicker() override
	{
		return Ticker;
	}

	virtual FSimpleMulticastDelegate& GetOnTickPreEngineCompleted() override
	{
		return OnTickPreEngineCompleted;
	}

	virtual void LockToTimecode(bool Locked) override
	{
		TimecodeLocked = Locked;
	}

	virtual void RegisterCaptureSupport(IMediaCaptureSupport& Support) override
	{
		CaptureSupports.AddUnique(&Support);
	}

	virtual void RegisterPlayerFactory(IMediaPlayerFactory& Factory) override
	{
		PlayerFactories.AddUnique(&Factory);
	}

	virtual void SetPlayerLifecycleManagerDelegate(IMediaPlayerLifecycleManagerDelegate* Delegate) override
	{
		PlayerLifecycleManagerDelegate = Delegate;
	}

	virtual IMediaPlayerLifecycleManagerDelegate* GetPlayerLifecycleManagerDelegate() override
	{
		return PlayerLifecycleManagerDelegate;
	}

	virtual uint64 CreateMediaPlayerInstanceID() override
	{
		uint64 InstanceID;
		while ((InstanceID = NextMediaPlayerInstanceID++) == ~0)
			;
		return InstanceID;
	}

	virtual void SetTimeSource(const TSharedPtr<IMediaTimeSource, ESPMode::ThreadSafe>& NewTimeSource) override
	{
		TimeSource = NewTimeSource;
	}

	virtual TSharedPtr<IMediaTimeSource, ESPMode::ThreadSafe> GetTimeSource() override
	{
		return TimeSource;
	}

	virtual void TickPostEngine() override
	{
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Media_TickFetch);
			Clock.TickFetch();
		}

		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Media_TickRender);
			Clock.TickRender();
		}
	}

	virtual void TickPostRender() override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Media_TickOutput);
		Clock.TickOutput();
	}

	virtual void TickPreEngine() override
	{
		FrameStartTime = FPlatformTime::Seconds();

		if (TimeSource.IsValid())
		{
			Clock.UpdateTimecode(TimeSource->GetTimecode(), TimecodeLocked);
		}

		QUICK_SCOPE_CYCLE_COUNTER(STAT_Media_TickInput);
		Clock.TickInput();

		OnTickPreEngineCompleted.Broadcast();
	}

	virtual void TickPreSlate() override
	{
		// currently not used
	}

	virtual void UnregisterCaptureSupport(IMediaCaptureSupport& Support) override
	{
		CaptureSupports.Remove(&Support);
	}

	virtual void UnregisterPlayerFactory(IMediaPlayerFactory& Factory) override
	{
		PlayerFactories.Remove(&Factory);
	}

	virtual double GetFrameStartTime() const override
	{
		return FrameStartTime;
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		if (!IsRunningDedicatedServer())
		{
			TickerThread = FRunnableThread::Create(&Ticker, TEXT("FMediaTicker"));
		}
	}

	virtual void ShutdownModule() override
	{
		if (TickerThread != nullptr)
		{
			TickerThread->Kill(true);
			delete TickerThread;
			TickerThread = nullptr;
		}

		CaptureSupports.Reset();
		PlayerFactories.Reset();
	}

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

private:

	/** The registered capture device support objects. */
	TArray<IMediaCaptureSupport*> CaptureSupports;

	/** The media clock. */
	FMediaClock Clock;

	/** Realtime at which frame started. */
	double FrameStartTime;

	/** Time code of the current frame. */
	FTimespan CurrentTimecode;

	/** The registered video player factories. */
	TArray<IMediaPlayerFactory*> PlayerFactories;

	/** Player lifecycle manager delegate */
	IMediaPlayerLifecycleManagerDelegate* PlayerLifecycleManagerDelegate = nullptr;

	/** MediaPlayer instance ID to hand out next */
	uint64 NextMediaPlayerInstanceID = 0;

	/** High-frequency ticker runnable. */
	FMediaTicker Ticker;

	/** High-frequency ticker thread. */
	FRunnableThread* TickerThread;

	/** Delegate to receive TickPreEngine */
	FSimpleMulticastDelegate OnTickPreEngineCompleted;

	/** Whether media objects should lock to the media clock's time code. */
	bool TimecodeLocked = false;

	/** The media clock's time source. */
	TSharedPtr<IMediaTimeSource, ESPMode::ThreadSafe> TimeSource;
};

IMPLEMENT_MODULE(FMediaModule, Media);
