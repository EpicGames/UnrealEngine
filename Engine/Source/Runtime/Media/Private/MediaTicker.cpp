// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaTicker.h"

#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ScopeLock.h"

#include "IMediaTickable.h"


/* FMediaTicker structors
 *****************************************************************************/

FMediaTicker::FMediaTicker()
	: Stopping(false)
{
	WakeupEvent = FPlatformProcess::GetSynchEventFromPool(true);
}


FMediaTicker::~FMediaTicker()
{
	FPlatformProcess::ReturnSynchEventToPool(WakeupEvent);
	WakeupEvent = nullptr;
}


/* FRunnable interface
 *****************************************************************************/

bool FMediaTicker::Init()
{
	return true;
}


uint32 FMediaTicker::Run()
{
	while (!Stopping)
	{
		if (WakeupEvent->Wait() && !Stopping)
		{
			TickTickables();
			if (!Stopping)
			{
				FPlatformProcess::Sleep(0.005f);
			}
		}
	}

	return 0;
}


void FMediaTicker::Stop()
{
	Stopping = true;
	WakeupEvent->Trigger();
}


void FMediaTicker::Exit()
{
	// do nothing
}


/* IMediaTicker interface
 *****************************************************************************/

void FMediaTicker::AddTickable(const TSharedRef<IMediaTickable, ESPMode::ThreadSafe>& Tickable)
{
	FScopeLock Lock(&CriticalSection);
	Tickables.AddUnique(Tickable);
	WakeupEvent->Trigger();
}


void FMediaTicker::RemoveTickable(const TSharedRef<IMediaTickable, ESPMode::ThreadSafe>& Tickable)
{
	FScopeLock Lock(&CriticalSection);
	Tickables.Remove(Tickable);
}


/* FMediaTicker implementation
 *****************************************************************************/

void FMediaTicker::TickTickables()
{
	TickablesCopy.Reset();
	{
		FScopeLock Lock(&CriticalSection);

		for (int32 TickableIndex = Tickables.Num() - 1; TickableIndex >= 0; --TickableIndex)
		{
			TSharedPtr<IMediaTickable, ESPMode::ThreadSafe> Tickable = Tickables[TickableIndex].Pin();

			if (Tickable.IsValid())
			{
				TickablesCopy.Add(Tickable);
			}
			else
			{
				Tickables.RemoveAtSwap(TickableIndex);
			}
		}

		if (Tickables.Num() == 0)
		{
			WakeupEvent->Reset();
		}
	}

	for (int32 i=0; i < TickablesCopy.Num(); ++i)
	{
		TSharedPtr<IMediaTickable, ESPMode::ThreadSafe> Tickable = TickablesCopy[i].Pin();

		if (Tickable.IsValid())
		{
			Tickable->TickTickable();
		}
	}
}
