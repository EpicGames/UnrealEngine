// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/CriticalSection.h"
#include "HAL/Runnable.h"
#include "Templates/Atomic.h"
#include "Templates/SharedPointer.h"

#include "IMediaTicker.h"

class FEvent;
class IMediaTickable;


/**
 * High frequency ticker thread.
 */
class FMediaTicker
	: public FRunnable
	, public IMediaTicker
{
public:

	/** Default constructor. */
	FMediaTicker();

	/** Virtual destructor. */
	virtual ~FMediaTicker();

public:

	//~ FRunnable interface

	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;

public:

	//~ IMediaTicker interface

	virtual void AddTickable(const TSharedRef<IMediaTickable, ESPMode::ThreadSafe>& Tickable) override;
	virtual void RemoveTickable(const TSharedRef<IMediaTickable, ESPMode::ThreadSafe>& Tickable) override;

protected:

	/** Tick all tickables. */
	void TickTickables();

private:

	/** Critical section for synchronizing access to high-frequency clock sinks. */
	FCriticalSection CriticalSection;

	/** Holds a flag indicating that the thread is stopping. */
	TAtomic<bool> Stopping;

	/** Collection of tickable objects. */
	TArray<TWeakPtr<IMediaTickable, ESPMode::ThreadSafe>> Tickables;

	/** Variable to avoid rellocating the Tickables array repeatedly. */
	TArray<TWeakPtr<IMediaTickable, ESPMode::ThreadSafe>> TickablesCopy;

	/** Holds an event signaling the thread to wake up. */
	FEvent* WakeupEvent;
};