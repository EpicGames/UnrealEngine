// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Optional.h"
#include "Misc/Timecode.h"
#include "Misc/Timespan.h"
#include "IMediaTimeSource.h"


/**
 * Available formats for media audio samples.
 */
enum class EMediaAudioSampleFormat
{
	/** Format not defined. */
	Undefined,

	/** Uncompressed 64-bit double precision floating point samples. */
	Double,

	/** Uncompressed 32-bit single precision floating point samples. */
	Float,

	/** Uncompressed 8-bit integer samples.*/
	Int8,

	/** Uncompressed 16-bit integer samples.*/
	Int16,

	/** Uncompressed 32-bit integer samples.*/
	Int32
};


/**
 * Interface for media audio samples.
 */
class IMediaAudioSample
{
public:

	/**
	 * Get the sample data.
	 *
	 * The returned buffer is only valid for the life time of this sample.
	 *
	 * @return Pointer to data buffer.
	 * @see GetChannels, GetDuration, GetFormat, GetFrames, GetRate, GetTime
	 */
	virtual const void* GetBuffer() = 0;

	/**
	 * Get the sample's number of channels.
	 *
	 * @return Number of channels.
	 * @see GetBuffer, GetDuration, GetFormat, GetFrames, GetRate, GetTime
	 */
	virtual uint32 GetChannels() const = 0;

	/**
	 * Get the amount of time for which the sample is valid.
	 *
	 * A duration of zero indicates that the sample is valid until the
	 * timecode of the next sample in the queue.
	 *
	 * @return Sample duration.
	 * @see GetBuffer, GetChannels, GetFormat, GetFrames, GetRate, GetTime
	 */
	virtual FTimespan GetDuration() const = 0;

	/**
	 * Get the audio sample format.
	 *
	 * @return Sample type.
	 * @see GetBuffer, GetChannels, GetDuration, GetFrames, GetRate, GetTime
	 */
	virtual EMediaAudioSampleFormat GetFormat() const = 0;

	/**
	 * Get the number of frames in the buffer.
	 *
	 * A frame consists of one sample value per channel.
	 *
	 * @return Frame count.
	 * @see GetBuffer, GetChannels, GetDuration, GetFormat, GetRate, GetTime
	 */
	virtual uint32 GetFrames() const = 0;

	/**
	 * Get the sample's sampling rate (in audio frames per second).
	 *
	 * @return Sample rate, i.e. 44100 Hz.
	 * @see GetBuffer, GetChannels, GetDuration, GetFormat, GetFrames, GetTime
	 */
	virtual uint32 GetSampleRate() const = 0;

	/**
	 * Get the sample time (in the player's local clock).
	 *
	 * This value is used primarily for debugging purposes.
	 *
	 * @return Sample time.
	 * @see GetBuffer, GetChannels, GetDuration, GetFormat, GetFrames, GetSampleRate
	 */
	virtual FMediaTimeStamp GetTime() const = 0;

	/**
	 * Get the sample timecode if available.
	 *
	 * @return Sample timecode.
	 * @see GetTime
	 */
	virtual TOptional<FTimecode> GetTimecode() const { return TOptional<FTimecode>(); }

public:

	/** Virtual destructor. */
	virtual ~IMediaAudioSample() = default;
};
