// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Optional.h"
#include "Misc/Timecode.h"
#include "Misc/Timespan.h"
#include "Misc/Guid.h"
#include "IMediaTimeSource.h"


/**
 * Interface for media binary data samples.
 */
class IMediaBinarySample
{
public:

	/**
	 * Get the sample data.
	 *
	 * @return Pointer to data buffer.
	 * @see GetDuration, GetSize, GetTime
	 */
	virtual const void* GetData() = 0;

	/**
	 * Get the amount of time for which the sample is valid.
	 *
	 * A duration of zero indicates that the sample is valid until the
	 * timecode of the next sample in the queue.
	 *
	 * @return Sample duration.
	 * @see GetData, GetSize, GetTime
	 */
	virtual FTimespan GetDuration() const = 0;

	/**
	 * Get the size of the binary data.
	 *
	 * @see GetData, GetDuration, GetTime
	 */
	virtual uint32 GetSize() const = 0;

	/**
	 * Get the sample time (in the player's local clock).
	 *
	 * This value is used primarily for debugging purposes.
	 *
	 * @return Sample time.
	 * @see GetData, GetDuration, GetSize, GetTime
	 */
	virtual FMediaTimeStamp GetTime() const = 0;

	/**
	 * Get the sample timecode if available.
	 *
	 * @return Sample timecode.
	 * @see GetTime
	 */
	virtual TOptional<FTimecode> GetTimecode() const { return TOptional<FTimecode>(); }


	/**
	 * Get the GUID identifying the derived type that may implement additional,
	 * type specific methods. If the GUID matches a type known type it is safe
	 * to static cast this class to the derived type.
	 * This is usually done to identify the format of the binary data carried here.
	 */
	virtual FGuid GetGUID() const { return FGuid(); }

public:

	/** Virtual destructor. */
	virtual ~IMediaBinarySample() = default;
};
