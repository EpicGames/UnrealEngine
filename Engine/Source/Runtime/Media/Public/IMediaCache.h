// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Range.h"
#include "Math/RangeSet.h"
#include "Misc/Timespan.h"


/**
 * Enumerates status types of media samples.
 *
 * @see IMediaSamples.GetSampleState
 */
enum class EMediaCacheState
{
	/** Sample has been cached (reserved for high-level caching; use Loaded in player plug-ins). */
	Cached,

	/** Sample has finished loading. */
	Loaded,

	/** Sample is currently being loaded. */
	Loading,

	/** Sample is scheduled to be loaded. */
	Pending,
};


/**
 * Interface for access to a media player's cache.
 *
 * @see IMediaControls, IMediaPlayer, IMediaSamples, IMediaTracks, IMediaView
 */
class IMediaCache
{
public:

	/**
	 * Query the time ranges of cached media samples for the specified caching state.
	 *
	 * This method can be used to probe a media player's decoder for which samples are
	 * scheduled for loading, being loaded, or finished loading. This is generally only
	 * supported by those players that expose some kind of internal sample caching or
	 * load/decode scheduling mechanism, and most players may simply ignore this call.
	 *
	 * @param State The sample state we're interested in.
	 * @param OutTimeRanges Will contain the set of matching sample time ranges.
	 * @return true on success, false if not supported.
	 */
	virtual bool QueryCacheState(EMediaCacheState State, TRangeSet<FTimespan>& OutTimeRanges) const
	{
		return false; // override in child classes, if supported
	}
	
	/**
	 * Query the number of samples for the specified caching state.
	 *
	 * @param State The sample state we're interested in.
	 * @return Sample count.
	 */
	virtual int32 GetSampleCount(EMediaCacheState State) const
	{
		return 0; // override in child classes, if supported
	}

public:

	/** Virtual destructor. */
	virtual ~IMediaCache() = default;
};
