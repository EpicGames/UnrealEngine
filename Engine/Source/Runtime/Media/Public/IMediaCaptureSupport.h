// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"


/**
 * Known capture device types.
 */
enum class EMediaCaptureDeviceType : uint32
{
	/** Unknown capture device type. */
	Unknown,

	/** Unspecified audio capture device. */
	Audio,

	/** Audio capture card. */
	AudioCard,

	/** Software audio capture device. */
	AudioSoftware,

	/** Depth sensors. */
	DepthSensor,

	/** Microphone. */
	Microphone,

	/** Unspecified video capture device. */
	Video,

	/** Video capture card. */
	VideoCard,

	/** Software video capture device. */
	VideoSoftware,

	/** Unspecified web cam. */
	Webcam,

	/** Front facing web cams. */
	WebcamFront,

	/** Rear facing web cams. */
	WebcamRear
};


/**
 * Information about a capture device.
 */
struct FMediaCaptureDeviceInfo
{
	/** Human readable display name. */
	FText DisplayName;

	/** Device specific debug information. */
	FString Info;

	/** The type of capture device. */
	EMediaCaptureDeviceType Type;

	/** Media URL string for use with media players. */
	FString Url;
};


/**
 * Interface for media capture support classes.
 */
class IMediaCaptureSupport
{
public:

	/**
	 * Enumerate available audio capture devices.
	 *
	 * @param OutDeviceInfos Will contain information about the devices.
	 * @see EnumerateVideoCaptureDevices
	 */
	virtual void EnumerateAudioCaptureDevices(TArray<FMediaCaptureDeviceInfo>& OutDeviceInfos) = 0;

	/**
	 * Enumerate available video capture devices.
	 *
	 * @param OutDeviceInfos Will contain information about the devices.
	 * @see EnumerateAudioCaptureDevices
	 */
	virtual void EnumerateVideoCaptureDevices(TArray<FMediaCaptureDeviceInfo>& OutDeviceInfos) = 0;

public:

	/** Virtual destructor. */
	virtual ~IMediaCaptureSupport() = default;
};
