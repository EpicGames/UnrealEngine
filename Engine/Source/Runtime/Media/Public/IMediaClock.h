// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"

class IMediaClockSink;


/**
 * Interface for media framework clocks.
 */
class IMediaClock
{
public:

	/**
	 * Get the clock's current time code.
	 *
	 * @return Time code.
	 * @see IsTimecodeLocked
	 */
	virtual FTimespan GetTimecode() const = 0;

	/**
	 * Whether the clock's time code is locked to an external time source.
	 *
	 * @return true if locked, false otherwise.
	 * @see GetTimecode
	 */
	virtual bool IsTimecodeLocked() const = 0;

public:

	/**
	 * Add a media clock sink.
	 *
	 * @param Sink The sink object to add.
	 * @see RemoveSink
	 */
	virtual void AddSink(const TSharedRef<IMediaClockSink, ESPMode::ThreadSafe>& Sink) = 0;

	/**
	 * Remove a media clock sink.
	 *
	 * @param Sink The sink object to remove.
	 * @see AddSink
	 */
	virtual void RemoveSink(const TSharedRef<IMediaClockSink, ESPMode::ThreadSafe>& Sink) = 0;

public:

	/** Virtual destructor. */
	virtual ~IMediaClock() = default;
};
