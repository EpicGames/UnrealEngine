// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Timespan.h"


/**
 * Interface for media clock sinks.
 *
 * This interface can be implemented by classes that wish to be ticked
 * by the Media Framework clock. The following tick stages are available
 * (in the order in which they are called on clock sinks):
 *
 * TickInput
 *    Called each tick from the main thread before the Engine is being ticked.
 *    It is used by media player plug-ins to update their state and initiate
 *    the reading of new input samples.
 *
 * TickFetch
 *    Called each tick from the main thread after the Engine has been ticked,
 *    but before TickRender. It can be used by media players to fetch the
 *    results of the TickInput stage prior to rendering.
 *
 * TickRender
 *    Called each tick from the main thread after TickFetch is complete, but
 *    before the frame has finished rendering. It is mainly used by media
 *    sinks to render the fetched input samples, such as drawing video frames
 *    to a texture or playing audio samples on a sound component.
 *
 * TickOutput
 *    Called each tick from the main thread after the Engine has been ticked
 *    and the frame finished rendering. It can be used by output plug-ins to
 *    write the completed frame to disk or stream it over the network.
 */
class IMediaClockSink
{
public:

	/**
	 * Called each tick to handle updates after the Engine ticked.
	 *
	 * @param DeltaTime Time since this function was last called.
	 * @param Timecode The current media time code.
	 * @see TickInput, TickOutput, TickRender
	 */
	virtual void TickFetch(FTimespan DeltaTime, FTimespan Timecode)
	{
		// override in child class if needed
	}

	/**
	 * Called each tick to handle input before the Engine is ticked.
	 *
	 * @param DeltaTime Time since this function was last called.
	 * @param Timecode The current media time code.
	 * @see TickFetch, TickOutput, TickRender
	 */
	virtual void TickInput(FTimespan DeltaTime, FTimespan Timecode)
	{
		// override in child class if needed
	}

	/**
	 * Called each tick to output the rendered frame.
	 *
	 * @param DeltaTime Time since this function was last called.
	 * @param Timecode The current media time code.
	 * @see TickFetch, TickInput, TickRender
	 */
	virtual void TickOutput(FTimespan DeltaTime, FTimespan Timecode)
	{
		// override in child class if needed
	}

	/**
	 * Called each tick before the frame finished rendering.
	 *
	 * @param DeltaTime Time since this function was last called.
	 * @param Timecode The current media time code.
	 * @see TickFetch, TickInput, TickOutput
	 */
	virtual void TickRender(FTimespan DeltaTime, FTimespan Timecode)
	{
		// override in child class if needed
	}

public:

	/** Virtual destructor. */
	virtual ~IMediaClockSink() = default;
};
