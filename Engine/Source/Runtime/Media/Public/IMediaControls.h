// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Range.h"
#include "Math/RangeSet.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Timespan.h"
#include "Misc/Optional.h"


/**
 * Available media controls.
 */
enum class EMediaControl
{
	/** Block on fetch. */
	BlockOnFetch,

	/** Pause playback. */
	Pause,

	/** Resume playback. */
	Resume,

	/** Seek to playback position (while updating output). */
	Scrub,

	/** Seek to playback position. */
	Seek,

	/** Playback of a range. */
	PlaybackRange
};


/**
 * Directions for seeking in media.
 */
enum class EMediaSeekDirection
{
	/** Seek backwards from current position. */
	Backward,

	/** Seek from the beginning of the media. */
	Beginning,

	/** Seek from the end of the media. */
	End,

	/** Seek forward from current position. */
	Forward
};


/**
 * Thinning modes for playback rates.
 */
enum class EMediaRateThinning
{
	/** Frames will be skipped to accommodate play rate. */
	Thinned,

	/** No frames will be skipped. */
	Unthinned
};


/**
 * Possible states of media playback.
 */
enum class EMediaState
{
	/** Media has been closed and cannot be played again. */
	Closed,

	/** Unrecoverable error occurred during playback. */
	Error,

	/** Playback has been paused, but can be resumed. */
	Paused,

	/** Media is currently playing. */
	Playing,

	/** Media is being prepared for playback. */
	Preparing,

	/** Playback has been stopped, but can be restarted. */
	Stopped
};


/**
 * Available media player status flags.
 */
enum class EMediaStatus
{
	/** No flags set. */
	None = 0x0,

	/** Player is buffering data. */
	Buffering = 0x1,

	/** Player is connecting to a media source. */
	Connecting = 0x2
};


ENUM_CLASS_FLAGS(EMediaStatus);


/**
 * Different types of media timeline ranges.
 */
enum class EMediaTimeRangeType
{
	/** Total absolute time range as defined by the media. */
	Absolute,

	/** Current time range of the media, set by media internal means or through API calls. */
	Current
};


/**
 * Additional parameters passed to Seek().
 */
struct FMediaSeekParams
{
	/** If set, the new sequence index a v2 timing enabled media player must be using now. */
	TOptional<int32> NewSequenceIndex;
};


/**
 * Interface for controlling media playback.
 *
 * @see IMediaCache, IMediaPlayer, IMediaSamples, IMediaTracks, IMediaView
 */
class IMediaControls
{
public:

	/**
	 * Whether the specified control is currently available.
	 *
	 * @return true if the control is available, false otherwise.
	 */
	virtual bool CanControl(EMediaControl Control) const = 0;

	/**
	 * Get the media's duration.
	 *
	 * @return A time span representing the duration.
	 * @see GetTime
	 */
	virtual FTimespan GetDuration() const = 0;

	/**
	 * Get the nominal playback rate, i.e. 1.0 for real time.
	 *
	 * @return Playback rate.
	 * @see Pause, Play, SetRate
	 */
	virtual float GetRate() const = 0;

	/**
	 * Get the state of the media.
	 *
	 * @return Media state.
	 * @see IsLooping
	 */
	virtual EMediaState GetState() const = 0;

	/**
	 * Get media player status flags.
	 *
	 * @return Status flags.
	 */
	virtual EMediaStatus GetStatus() const = 0;

	/**
	 * Get the supported playback rates.
	 *
	 * @param Thinning The desired rate thinning mode.
	 * @return The ranges of supported rates.
	 * @see SetRate, SupportsRate
	 */
	virtual TRangeSet<float> GetSupportedRates(EMediaRateThinning Thinning) const = 0;

	/**
	 * Get the player's current playback time.
	 *
	 * @return Playback time.
	 * @see Seek
	 */
	virtual FTimespan GetTime() const = 0;

	/**
	 * Check whether playback is currently looping.
	 *
	 * @return true if playback is looping, false otherwise.
	 * @see GetState, SetLooping
	 */
	virtual bool IsLooping() const = 0;

	/**
	 * Change the media's playback time.
	 *
	 * @param Time The playback time to set.
	 * @return true on success, false otherwise.
	 * @see GetTime
	 */
	virtual bool Seek(const FTimespan& Time) = 0;

	/**
	 * Set whether playback should be looping.
	 *
	 * @param Looping Enables or disables looping.
	 * @see IsLooping
	 */
	virtual bool SetLooping(bool Looping) = 0;

	/**
	 * Set the current playback rate.
	 *
	 * A playback rate of 1.0 will play the media normally at real-time.
	 * A rate of 0.0 corresponds to pausing playback. A negative rate, if
	 * supported, plays the media in reverse, and a rate larger than 1.0
	 * fast forwards playback.
	 *
	 * @param Rate The playback rate to set.
	 * @return true on success, false otherwise.
	 * @see GetRate, Pause, Play
	 */
	virtual bool SetRate(float Rate) = 0;

	/**
	 * Hint for player indicating that blocked playback mode will be used / not used
	 *
	 * @param bFacadeWillUseBlockingPlayback True if blocked playback will be used, false otherwiese
	 *
	 * @note Implement as needed by the player.
	 */
	virtual void SetBlockingPlaybackHint(bool bFacadeWillUseBlockingPlayback)
	{
	}


	/**
	 * Same as Seek(), but with additional options.
	 * This default implementation just calls Seek() and ignores options.
	 */
	virtual bool Seek(const FTimespan& InNewTime, const FMediaSeekParams& InAdditionalParams)
	{
		return Seek(InNewTime);
	}

public:

	/**
	 * Pause media playback.
	 *
	 * This is the same as setting the playback rate to 0.0.
	 *
	 * @return true if the media is being paused, false otherwise.
	 * @see Play, Stop
	 */
	inline bool Pause()
	{
		return SetRate(0.0f);
	}

	/**
	 * Start media playback at the default rate of 1.0.
	 *
	 * This is the same as setting the playback rate to 1.0.
	 *
	 * @return true if playback is starting, false otherwise.
	 * @see Pause, Stop
	 */
	inline bool Play()
	{
		return SetRate(1.0f);
	}

	/**
	 * Change the playback time of the media by a relative offset in the given direction.
	 *
	 * @param TimeOffset The offset to apply to the time.
	 * @param Direction The direction to seek in.
	 * @return true on success, false otherwise.
	 * @see GetDuration, GetTime
	 */
	UE_DEPRECATED(5.6, "Use absolute seeking only. This method will be removed soon.")
	bool Seek(const FTimespan& TimeOffset, EMediaSeekDirection Direction)
	{
		FTimespan SeekTime;

		switch (Direction)
		{
		case EMediaSeekDirection::Backward:
			SeekTime = GetTime() - TimeOffset;
			break;

		case EMediaSeekDirection::Beginning:
			SeekTime = TimeOffset;
			break;

		case EMediaSeekDirection::End:
			SeekTime = GetDuration() - TimeOffset;
			break;

		case EMediaSeekDirection::Forward:
			SeekTime = GetTime() + TimeOffset;
			break;
		}

		return Seek(SeekTime);
	}

	virtual TRange<FTimespan> GetPlaybackTimeRange(EMediaTimeRangeType InRangeToGet) const
	{
		return TRange<FTimespan>(FTimespan(0), GetDuration());
	}

	virtual bool SetPlaybackTimeRange(const TRange<FTimespan>& InTimeRange)
	{
		return false;
	}

public:

	/** Virtual destructor. */
	virtual ~IMediaControls() = default;
};
