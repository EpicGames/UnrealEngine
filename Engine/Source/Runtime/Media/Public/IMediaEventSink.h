// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/**
 * Enumerates media player related events.
 */
enum class EMediaEvent
{
	/** The media source started buffering data. */
	MediaBuffering,

	/** The media source completed buffering data. */
	MediaBufferingComplete,

	/** The current media source has been closed. */
	MediaClosed,

	/** The player started connecting to the media source. */
	MediaConnecting,

	/** A new media source has been opened. */
	MediaOpened,

	/** A media source failed to open. */
	MediaOpenFailed,

	/** The end of the media (or beginning if playing in reverse) has been reached. */
	PlaybackEndReached,

	/** Playback has been resumed. */
	PlaybackResumed,

	/** Playback has been suspended. */
	PlaybackSuspended,

	/** Seek operation has completed successfully. */
	SeekCompleted,

	/** Media tracks have changed. */
	TracksChanged,

	/** Media metadata has changed. */
	MetadataChanged,

	// - - - - - - - - - - - - - - - - - - - - - - - -

	/** Start of internal events not sent out to user code. */
	Internal_Start,

	/** Player requests purge of outdated video samples from queues. */
	Internal_PurgeVideoSamplesHint = Internal_Start,

	/** Player notifies about availability or unavailablity of media samples. **/
	Internal_VideoSamplesAvailable,
	Internal_VideoSamplesUnavailable,
	Internal_AudioSamplesAvailable,
	Internal_AudioSamplesUnavailable,

	/**
	 * Player requests the facade to clear its "last sample still good" cache so the next
	 * TickInput fetches and delivers a fresh sample even if the playback time range has
	 * not advanced.
	 */
	Internal_RequestVideoSampleRefresh,
};


/**
 * Interface for classes that receive media player events.
 */
class IMediaEventSink
{
public:

	/**
	 * Receive the given media event.
	 *
	 * @param Event The event to receive.
	 * @note Implementors must be thread-safe as this method may be called from arbitrary worker threads.
	 */
	virtual void ReceiveMediaEvent(EMediaEvent Event) = 0;

public:

	/** Virtual destructor. */
	virtual ~IMediaEventSink() = default;
};
