// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/Variant.h"

/**
 * This structure describes a metadata item.
 */
class IMediaMetadataItem
{
public:
	virtual ~IMediaMetadataItem() = default;

	/*
	  Language this item is described in.
	  This should a three letter ISO 639-2 code. If no language is specified this item is intended
	  to be used when no other item exists in the user's preferred language.
	*/
	virtual const FString& GetLanguageCode() const = 0;

	/*
	  Optional mime type of the item described by Value.
	  This should be set when Value is of type EVariantTypes::ByteArray
	  For example, if this item is a binary JPEG image the mime type should be set as "image/jpeg"
	  to identify the type of the payload.
	*/
	virtual const FString& GetMimeType() const = 0;

	/*
	  The value of the metadata item.
	*/
	virtual const FVariant& GetValue() const = 0;
};
