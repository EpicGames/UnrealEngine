// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Misc/Timespan.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class FName;
class IMediaCaptureSupport;
class IMediaClock;
class IMediaPlayerFactory;
class IMediaTicker;
class IMediaTimeSource;
class IMediaPlayerLifecycleManagerDelegate;

/**
 * Interface for the Media module.
 *
 * Media Framework is ticked in several stages. The Input tick happens after the Engine
 * (including Sequencer, game world, and Blueprints) were ticked. This allows any game
 * logic to modify the state of tickable media objects before they fetch input.
 *
 * The Update tick happens right after the Input tick when all tickable media objects
 * have fetched the latest input based on their current state.
 *
 * Finally, the Output tick happens after all game and core ticking completed, and
 * after all render commands have been enqueued. It is the very last thing to happen
 * before the frame is complete.
 */
class IMediaModule
	: public IModuleInterface
{
public:

	//~ Platform management

	/*
	 * Get a nice platform name from a GUID
	*/
	virtual FName GetPlatformName(const FGuid& PlatformGuid) const = 0;

	/*
	 * Get a GUID from a nice platform name
	*/
	virtual FGuid GetPlatformGuid(const FName & PlatformName) const = 0;

	//~ Capture devices

	/**
	 * Get all registered capture device support objects.
	 *
	 * @return Collection of registered objects.
	 * @see RegisterCaptureDevices, UnregisterCaptureDevices
	 */
	virtual const TArray<IMediaCaptureSupport*>& GetCaptureSupports() const = 0;

	/**
	 * Register a media capture devices support object.
	 *
	 * @param Support The support object to register.
	 * @see GetCaptureDevices, UnregisterCaptureDevices
	 */
	virtual void RegisterCaptureSupport(IMediaCaptureSupport& Support) = 0;

	/**
	 * Unregister a media capture device support object.
	 *
	 * @param Support The support object to unregister.
	 * @see GetCaptureDevices, RegisterCaptureDevices
	 */
	virtual void UnregisterCaptureSupport(IMediaCaptureSupport& Support) = 0;

public:

	//~ Player factories

	/**
	 * Get the list of installed media player factories.
	 *
	 * @return Collection of media player factories.
	 * @see GetPlayerFactory, RegisterPlayerFactory, UnregisterPlayerFactory
	 */
	virtual const TArray<IMediaPlayerFactory*>& GetPlayerFactories() const = 0;

	/**
	 * Get a media player factory by name.
	 *
	 * @param FactoryName The name of the factory.
	 * @return The factory, or nullptr if not found.
	 * @see GetPlayerFactories, RegisterPlayerFactory, UnregisterPlayerFactory
	 */
	virtual IMediaPlayerFactory* GetPlayerFactory(const FName& FactoryName) const = 0;

	/**
	 * Get a media player factory by GUID.
	 *
	 * @param FactoryGuid The GUID of the factory / player.
	 * @return The factory, or nullptr if not found.
	 * @see GetPlayerFactories, RegisterPlayerFactory, UnregisterPlayerFactory
	 */
	virtual IMediaPlayerFactory* GetPlayerFactory(const FGuid& PlayerPluginGuid) const = 0;

	/**
	 * Register a media player factory.
	 *
	 * @param Factory The media player factory to register.
	 * @see GetPlayerFactories, RegisterPlayerFactory, UnregisterPlayerFactory
	 */
	virtual void RegisterPlayerFactory(IMediaPlayerFactory& Factory) = 0;

	/**
	 * Unregister a media player factory.
	 *
	 * @param Factory The media player factory to unregister.
	 * @see GetPlayerFactories, RegisterPlayerFactory, RegisterPlayerFactory
	 */
	virtual void UnregisterPlayerFactory(IMediaPlayerFactory& Factory) = 0;

	/**
	* Set player lifetime delegate
	* @param Delegate Delegate instance to set for use. Use nullptr to reset delegate.
	*/
	virtual void SetPlayerLifecycleManagerDelegate(IMediaPlayerLifecycleManagerDelegate* Delegate) = 0;

	/**
	* Get player lifetime delegate
	*/
	virtual IMediaPlayerLifecycleManagerDelegate* GetPlayerLifecycleManagerDelegate() = 0;

	/**
	* Get new media player instance ID
	*/
	virtual uint64 CreateMediaPlayerInstanceID() = 0;

public:

	//~ Time management

	/**
	 * Get the media clock.
	 *
	 * @return Media clock.
	 * @see GetTicker, LockToTimecode
	 */
	virtual IMediaClock& GetClock() = 0;

	/**
	 * Get the high-frequency ticker.
	 *
	 * @return The ticker.
	 * @see GetClock
	 */
	virtual IMediaTicker& GetTicker() = 0;

	/**
	 * Get frame's processing approximate real time start time in seconds
	 */
	virtual double GetFrameStartTime() const = 0;

	/**
	 * Get a Delegate that is trigger once all MediaClockSink are TickInput
	 *
	 * @return the OnTickPreEngineCompleted
	 */
	virtual FSimpleMulticastDelegate& GetOnTickPreEngineCompleted() = 0;

	/**
	 * Whether media objects should lock to the media clock's time code.
	 *
	 * Time code locking changes will take effect next frame.
	 *
	 * @param Locked true to enable time code locking, false to disable.
	 * @see GetClock
	 */
	virtual void LockToTimecode(bool Locked) = 0;

	/**
	 * Set the time source for the media clock.
	 *
	 * @param NewTimeSource The time source to set.
	 */
	virtual void SetTimeSource(const TSharedPtr<IMediaTimeSource, ESPMode::ThreadSafe>& NewTimeSource) = 0;


	/**
	 * Get the time source for the media clock.
	 *
	 * @return The current time source.
	 */
	virtual TSharedPtr<IMediaTimeSource, ESPMode::ThreadSafe> GetTimeSource() = 0;

	/**
	 * Called by the main loop after the game engine has been ticked.
	 *
	 * @see TickPostRender, TickPreEngine, TickPreSlate
	 */
	virtual void TickPostEngine() = 0;

	/**
	 * Called by the main loop after the entire frame has been rendered.
	 *
	 * @see TickPostEngine, TickPreEngine, TickPreSlate
	 */
	virtual void TickPostRender() = 0;

	/**
	 * Called by the main loop before the game engine is ticked.
	 *
	 * @see TickPostEngine, TickPostRender, TickPreSlate
	 */
	virtual void TickPreEngine() = 0;

	/**
	 * Called by the main loop before Slate is ticked.
	 *
	 * @see TickPostEngine, TickPostRender, TickPreEngine
	 */
	virtual void TickPreSlate() = 0;

public:

	/** Virtual destructor. */
	virtual ~IMediaModule() = default;
};
