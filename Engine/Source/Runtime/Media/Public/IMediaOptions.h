// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"


/**
 * Interface for media options.
 */
class IMediaOptions
{
public:

	class FDataContainer : public TSharedFromThis<FDataContainer, ESPMode::ThreadSafe>
	{
	public:
		virtual ~FDataContainer() = default;
	};

	/**
	 * Get the name of the desired native player.
	 *
	 * @return Native player name, or NAME_None to auto select.
	 */
	virtual FName GetDesiredPlayerName() const = 0;

	/**
	 * Get a Boolean media option.
	 *
	 * @param Key The name of the option to get.
	 * @param DefaultValue The default value to return if the option is not set.
	 * @return The option value.
	 */
	virtual bool GetMediaOption(const FName& Key, bool DefaultValue) const = 0;

	/**
	 * Get a double precision floating point media option.
	 *
	 * @param Key The name of the option to get.
	 * @param DefaultValue The default value to return if the option is not set.
	 * @return The option value.
	 */
	virtual double GetMediaOption(const FName& Key, double DefaultValue) const = 0;

	/**
	 * Get a signed integer media option.
	 *
	 * @param Key The name of the option to get.
	 * @param DefaultValue The default value to return if the option is not set.
	 * @return The option value.
	 */
	virtual int64 GetMediaOption(const FName& Key, int64 DefaultValue) const = 0;

	/**
	 * Get a string media option.
	 *
	 * @param Key The name of the option to get.
	 * @param DefaultValue The default value to return if the option is not set.
	 * @return The option value.
	 */
	virtual FString GetMediaOption(const FName& Key, const FString& DefaultValue) const = 0;

	/**
	 * Get a localized text media option.
	 *
	 * @param Key The name of the option to get.
	 * @param DefaultValue The default value to return if the option is not set.
	 * @return The option value.
	 */
	virtual FText GetMediaOption(const FName& Key, const FText& DefaultValue) const = 0;


	/**
	 * Get a complex data type media option.
	 *
	 * @param Key The name of the option to get.
	 * @param DefaultValue The default value to return if the option is not set.
	 * @return The option value.
	 */
	virtual TSharedPtr<FDataContainer, ESPMode::ThreadSafe> GetMediaOption(const FName& Key, const TSharedPtr<FDataContainer, ESPMode::ThreadSafe>& DefaultValue) const
	{
		return DefaultValue;
	}

	/**
	 * Check whether the specified option is set.
	 *
	 * @param Key The name of the option to check.
	 * @return true if the option is set, false otherwise.
	 */
	virtual bool HasMediaOption(const FName& Key) const = 0;

	/**
	 * @return a object that can be used by the UObject garbage collection system to keep this object alive.
	 * Players should not store a pointer of the interface but some players use the interface in async functionality and the object needs to be kept alive during those.
	 */
	virtual const UObject* ToUObject() const = 0;
};
