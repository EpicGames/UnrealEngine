// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"
#include "Misc/Timecode.h"
#include "Misc/Timespan.h"
#include "IMediaTimeSource.h"

/**
 * Type of text overlay samples.
 */
enum class EMediaOverlaySampleType
{
	/** Caption text for hearing impaired users. */
	Caption,

	/** Subtitle text for non-native speakers. */
	Subtitle,

	/** Generic text. */
	Text,
};


/**
 * Interface for media overlay text samples.
 */
class IMediaOverlaySample
{
public:

	/**
	 * Get the amount of time for which the sample should be displayed.
	 *
	 * @return Sample duration.
	 */
	virtual FTimespan GetDuration() const = 0;

	/**
	 * Get the position at which to display the text.
	 *
	 * @return Display position (relative to top-left corner, in pixels).
	 * @see GetText
	 */
	virtual TOptional<FVector2D> GetPosition() const = 0;

	/**
	 * Get the sample's text.
	 *
	 * @return The overlay text.
	 * @see GetPosition, GetType
	 */
	virtual FText GetText() const = 0;

	/**
	 * Get the sample time (in the player's local clock).
	 *
	 * This value is used primarily for debugging purposes.
	 *
	 * @return Sample time.
	 */
	virtual FMediaTimeStamp GetTime() const = 0;

	/**
	 * Get the sample timecode if available.
	 *
	 * @return Sample timecode.
	 * @see GetTime
	 */
	virtual TOptional<FTimecode> GetTimecode() const { return TOptional<FTimecode>(); }

	/**
	 * Get the sample type.
	 *
	 * @return Sample type.
	 * @see GetText
	 */
	virtual EMediaOverlaySampleType GetType() const = 0;

	/**
	 * Get the GUID identifying the derived type that may implement additional,
	 * type specific methods. If the GUID matches a type known type it is safe
	 * to static cast this class to the derived type.
	 */
	virtual FGuid GetGUID() const { return FGuid(); }

public:

	/** Virtual destructor. */
	virtual ~IMediaOverlaySample() = default;

};
