// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Internationalization/Text.h"
#include "Math/Interval.h"
#include "Math/IntPoint.h"
#include "Math/MathFwd.h"
#include "Misc/Paths.h"
#include "Misc/Timespan.h"
#include "Misc/Variant.h"

class FArchive;

class IMediaCache;
class IMediaControls;
class IMediaOptions;
class IMediaSamples;
class IMediaTracks;
class IMediaView;
class IMediaMetadataItem;

struct FGuid;
struct FMediaPlayerOptions;

/**
 * Interface for media players.
 *
 * @see IMediaPlayerFactory
 */
class IMediaPlayer
{
public:

	//~ The following methods must be implemented by media players

	/**
	 * Close a previously opened media source.
	 *
	 * Call this method to free up all resources associated with an opened
	 * media source. If no media is open, this function has no effect.
	 *
	 * The media may not necessarily be closed after this function succeeds,
	 * because closing may happen asynchronously. Subscribe to the MediaClosed
	 * event to detect when the media finished closing. This events is only
	 * triggered if Close returns true.
	 *
	 * @see IsReady, Open
	 */
	virtual void Close() = 0;

	/**
	 * Get the player's cache controls.
	 *
	 * The interface returned by this method must remain valid for the player's life time.
	 *
	 * @return Cache controls.
	 * @see GetControls, GetSamples, GetTracks, GetView
	 */
	virtual IMediaCache& GetCache() = 0;

	/**
	 * Get the player's playback controls.
	 *
	 * The interface returned by this method must remain valid for the player's life time.
	 *
	 * @return Playback controls.
	 * @see GetCache, GetSamples, GetTracks, GetView
	 */
	virtual IMediaControls& GetControls() = 0;

	/**
	 * Get debug information about the player and currently opened media.
	 *
	 * @return Information string.
	 * @see GetStats
	 */
	virtual FString GetInfo() const = 0;

	/**
	 * Get the GUID for this player plugin.
	 *
	 * @return Media player GUID (usually corresponds to a player name)
	 * @see GetPlayerName
	 */
	virtual FGuid GetPlayerPluginGUID() const = 0;

	/**
	 * Get the player's sample queue.
	 *
	 * The interface returned by this method must remain valid for the player's life time.
	 *
	 * @return Cache interface.
	 * @see GetCache, GetControls, GetTracks, GetView
	 */
	virtual IMediaSamples& GetSamples() = 0;

	/**
	 * Get playback statistics information.
	 *
	 * @return Information string.
	 * @see GetInfo
	 */
	virtual FString GetStats() const = 0;

	/**
	 * Get the player's track collection.
	 *
	 * The interface returned by this method must remain valid for the player's life time.
	 *
	 * @return Tracks interface.
	 * @see GetCache, GetControls, GetSamples, GetView
	 */
	virtual IMediaTracks& GetTracks() = 0;

	/**
	 * Get the URL of the currently loaded media.
	 *
	 * @return Media URL.
	 */
	virtual FString GetUrl() const = 0;

	/**
	 * Get the player's view settings.
	 *
	 * The interface returned by this method must remain valid for the player's life time.
	 *
	 * @return View interface.
	 * @see GetCache, GetControls, GetSamples, GetTracks
	 */
	virtual IMediaView& GetView() = 0;

	/**
	 * Open a media source from a URL with optional parameters.
	 *
	 * The media may not necessarily be opened after this function succeeds,
	 * because opening may happen asynchronously. Subscribe to the MediaOpened
	 * and MediaOpenFailed events to detect when the media finished or failed
	 * to open. These events are only triggered if Open returns true.
	 *
	 * The optional parameters can be used to configure aspects of media playback
	 * and are specific to the type of media source and the underlying player.
	 * Check their documentation for available keys and values.
	 *
	 * @param Url The URL of the media to open (file name or web address).
	 * @param Options Optional media parameters.
	 * @return true if the media is being opened, false otherwise.
	 * @see Close, IsReady, OnOpen, OnOpenFailed
	 */
	virtual bool Open(const FString& Url, const IMediaOptions* Options) = 0;

	/**
	 * Open a media source from a file or memory archive with optional parameters.
	 *
	 * The media may not necessarily be opened after this function succeeds,
	 * because opening may happen asynchronously. Subscribe to the MediaOpened
	 * and MediaOpenFailed events to detect when the media finished or failed
	 * to open. These events are only triggered if Open returns true.
	 *
	 * The optional parameters can be used to configure aspects of media playback
	 * and are specific to the type of media source and the underlying player.
	 * Check their documentation for available keys and values.
	 *
	 * @param Archive The archive holding the media data.
	 * @param OriginalUrl The original URL of the media that was loaded into the buffer.
	 * @param Options Optional media parameters.
	 * @return true if the media is being opened, false otherwise.
	 * @see Close, IsReady, OnOpen, OnOpenFailed
	 */
	virtual bool Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* Options) = 0;

public:

	//~ The following methods are optional

	/**
	 * Open a media source from a URL with optional asset and player parameters.
	 *
	 */
	virtual bool Open(const FString& Url, const IMediaOptions* Options, const FMediaPlayerOptions* PlayerOptions)
	{
		return Open(Url, Options);
	}

	/**
	 * Get information about the media that is playing.
	 *
	 * @param	InfoName		Name of the information we want.
	 * @returns					Requested information, or empty if not available.
	 * @see						UMediaPlayer::GetMediaInfo.
	 */
	virtual FVariant GetMediaInfo(FName InfoName) const
	{
		return FVariant();
	}

	/**
	 * Gets the current metadata of the media source.
	 * 
	 * Metadata is optional and if present is typically a collection of key/value items without a
	 * well defined meaning. It may contain information on copyright, album name, artist and such,
	 * but the availability of any item is not mandatory and the representation will vary with the
	 * type of media.
	 * Interpretation therefor requires the application to be aware of the type of media being loaded
	 * and the possible metadata it may carry.
	 * 
	 * Metadata may change over time. Its presence or change is reported by a MetadataChanged event.
	 */
	virtual TSharedPtr<TMap<FString, TArray<TUniquePtr<IMediaMetadataItem>>>, ESPMode::ThreadSafe> GetMediaMetadata() const
	{
		return nullptr;
	}

	/**
	 * Get the human readable name of the currently loaded media source.
	 *
	 * Depending on the type of media source, this might be the name of a file,
	 * the display name of a capture device, or some other identifying string.
	 * If the player does not provide a specialized implementation for this
	 * method, the media name will be derived from the current media URL.
	 *
	 * @return Media source name, or empty text if no media is opened
	 * @see GetPlayerName, GetUrl
	 */
	virtual FText GetMediaName() const
	{
		const FString Url = GetUrl();

		if (Url.IsEmpty())
		{
			return FText::GetEmpty();
		}

		return FText::FromString(FPaths::GetBaseFilename(Url));
	}

	/**
	 * Set the player's globally unique identifier.
	 *
	 * @param Guid The GUID to set.
	 */
	virtual void SetGuid(const FGuid& Guid)
	{
		// override in child classes if supported
	}

	/**
	 * Set the player's native volume if supported.
	 *
	 * @param Volume The volume to set.
	 * @return true on success, false otherwise.
	 */
	virtual bool SetNativeVolume(float Volume)
	{
		return false;
	}

	/**
	 * Notify player of last sample time of audio used.
	 *
	 * @param SampleTime The last audio sample dequeued by one of the audio sinks.
	 */
	virtual void SetLastAudioRenderedSampleTime(FTimespan SampleTime)
	{
		// override in child classes if supported
	}

	/**
	 * Tick the player's audio related code.
	 *
	 * This is a high-frequency tick function. Media players override this method
	 * to fetch and process audio samples, or to perform other time critical tasks.
	 *
	 * @see TickInput, TickFetch
	 */
	virtual void TickAudio()
	{
		// override in child class if needed
	}

	/**
	 * Tick the player in the Fetch phase.
	 *
	 * Media players may override this method to fetch newly decoded input
	 * samples before they are rendered on textures or audio components.
	 *
	 * @param DeltaTime Time since last tick.
	 * @param Timecode The current media time code.
	 * @see TickAudio, TickInput
	 */
	virtual void TickFetch(FTimespan DeltaTime, FTimespan Timecode)
	{
		// override in child class if needed
	}

	/**
	 * Tick the player in the Input phase.
	 *
	 * Media players may override this method to update their state before the
	 * Engine is being ticked, or to initiate the processing of input samples.
	 *
	 * @param DeltaTime Time since last tick.
	 * @param Timecode The current media time code.
	 * @see TickAudio, TickFetch
	 */
	virtual void TickInput(FTimespan DeltaTime, FTimespan Timecode)
	{
		// override in child class if needed
	}

	/**
	 * Flush sinks when seek begins
	 *
	 * @return true if sinks should be flushed when a seek starts
	 */
	virtual bool FlushOnSeekStarted() const
	{
		return false;
	}

	/**
	 * Flush sinks when seek ends
	 *
	 * @return true if sinks should be flushed when a seek finishes
	 */
	virtual bool FlushOnSeekCompleted() const
	{
		return true;
	}

	enum class EFeatureFlag
	{
		AllowShutdownOnClose = 0,		//!< Allow player to be shutdown right after 'close' event is received from it
		UsePlaybackTimingV2,			//!< Use v2 playback timing and AV sync
		UseRealtimeWithVideoOnly,		//!< Use realtime rather then game deltatime to control video playback if no audio is present
		AlwaysPullNewestVideoFrame,		//!< Mediaframework will not gate video frame output with its own timing, but assumes "ASAP" as output time for every sample
		PlayerUsesInternalFlushOnSeek,	//!< The player implements an internal flush logic on seeks and Mediaframework will not issue an explicit Flush() call to it on seeks
		IsTrackSwitchSeamless,			//!< If track switching is seamless then a flush of sinks is not necessary.
		PlayerSelectsDefaultTracks,		//!< Whether or not the player selects suitable track defaults.
	};
	
	virtual bool GetPlayerFeatureFlag(EFeatureFlag /*flag*/) const
	{
		// Override in child class if needed.
		return false;
	}

	class IAsyncResourceReleaseNotification
	{
	public:
		virtual ~IAsyncResourceReleaseNotification() = default;
		virtual void Signal(uint32 ResourceFlags) = 0;
	};
	typedef TSharedRef<IAsyncResourceReleaseNotification, ESPMode::ThreadSafe> IAsyncResourceReleaseNotificationRef;

	/**
	 * Set async resource release notification for use with IMediaPlayerLifecycleManagerDelegate
	 */
	virtual bool SetAsyncResourceReleaseNotification(IAsyncResourceReleaseNotificationRef AsyncDestructNotification)
	{
		// Override in child class if needed.
		return false;
	}

	/*
	* Return IMediaPlayerLifecycleManagerDelegate::ResourceFlags bitmask to indicate resource types recreated on a open call
	*/
	virtual uint32 GetNewResourcesOnOpen() const
	{
		// Override in child class if needed. Default assumes resources carry over (to a large degree) and are created per instance only
		// (as a simplified model for older players)
		return 0;
	}

#if WITH_EDITOR
	/** Records a source opened event to engine analytics, called when a player has successfully opened a media source */
	virtual void RecordSourceOpenedEvent() const { } 
	
	/**
	 * Gets whether a source opened event should be automatically recorded after Open is called and returns true on a media player.
	 * Media players that operate asynchronously should set this to false and call RecordSourceOpenedEvent manually when appropriate.
	 */
	virtual bool ShouldAutoRecordSourceOpenedEvent() const { return true; }
#endif

public:

	/** Virtual destructor. */
	virtual ~IMediaPlayer() = default;
};
