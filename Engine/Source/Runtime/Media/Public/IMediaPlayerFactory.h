// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

class FString;
class IMediaEventSink;
class IMediaOptions;
class IMediaPlayer;


/**
 * Enumerates available media player features.
 */
enum class EMediaFeature
{
	/** Audio output via Engine. */
	AudioSamples,

	/** Audio tracks. */
	AudioTracks,

	/** Caption tracks. */
	CaptionTracks,

	/** Metadata tracks (implies output via Engine). */
	MetadataTracks,

	/** Captions, subtitle and text output via Engine. */
	OverlaySamples,

	/** Subtitle tracks. */
	SubtitleTracks,

	/** Generic text tracks. */
	TextTracks,

	/** 360 degree video controls. */
	Video360,

	/** Video output via Engine. */
	VideoSamples,

	/** Stereoscopic video controls. */
	VideoStereo,

	/** Video tracks. */
	VideoTracks
};


/**
 * Interface for media player factories.
 *
 * Media player factories are used to create instances of media player implementations.
 * Most media players will be implemented inside plug-ins, which will register their
 * factories on startup. The Media module will use the CanPlayUrl() method on this
 * interface to determine which media player to instantiate for a given media source.
 *
 * @see IMediaPlayer
 */
class IMediaPlayerFactory
{
public:

	/**
	 * Whether the player can play the specified source URL.
	 *
	 * @param Url The media source URL to check.
	 * @param Options Optional media player parameters.
	 * @param OutWarnings Will contain warning messages (optional).
	 * @param OutErrors will contain error messages (optional).
	 * @return true if the source can be played, false otherwise.
	 */
	virtual bool CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const = 0;

	/**
	 * Creates a media player.
	 *
	 * @param EventSink The object that will receive the player's events.
	 * @return A new media player, or nullptr if a player couldn't be created.
	 */
	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) = 0;

	/**
	 * Get the human readable name of the player.
	 *
	 * @return Display name text.
	 * @see GetName
	 */
	virtual FText GetDisplayName() const = 0;

	/**
	 * Get the unique name of the media player.
	 *
	 * @return Media player name, i.e. 'AndroidMedia' or 'WmfMedia'.
	 * @see GetDisplayName
	 */
	virtual FName GetPlayerName() const = 0;

	/**
	 * Get the GUID for this player plugin.
	 *
	 * @return Media player GUID (usually corresponds to a player name)
	 * @see GetPlayerName
	 */
	virtual FGuid GetPlayerPluginGUID() const = 0;

	/**
	 * Get the names of platforms that the media player supports.
	 *
	 * The returned platforms names must match the ones returned by
	 * FPlatformProperties::IniPlatformName, i.e. "Windows", "Android", etc.
	 *
	 * @return Platform name collection.
	 * @see GetOptionalParameters, GetRequiredParameters, GetSupportedFileExtensions, GetSupportedUriSchemes
	 */
	virtual const TArray<FString>& GetSupportedPlatforms() const = 0;

	/**
	 * Check whether the media player supports the specified feature.
	 *
	 * @param Feature The feature to check.
	 * @return true if the feature is supported, false otherwise.
	 */
	virtual bool SupportsFeature(EMediaFeature Feature) const = 0;

public:

	/**
	 * Whether the player can play the specified source URL.
	 *
	 * @param Url The media source URL to check.
	 * @param Options Optional media player parameters.
	 * @return true if the source can be played, false otherwise.
	 */
	bool CanPlayUrl(const FString& Url, const IMediaOptions* Options) const
	{
		return CanPlayUrl(Url, Options, nullptr, nullptr);
	}

	/**
	 * Returns a confidence score how sure the player thinks it is to be able to play the specified source URL.
	 *
	 * @param Url The media source URL to check.
	 * @param Options Optional media player parameters.
	 * @param OutWarnings Will contain warning messages (optional).
	 * @param OutErrors will contain error messages (optional).
	 * @return A score giving a probability (0-100 (or higher if necessary)) whether or not the media can be played.
	 */
	virtual int32 GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const
	{
		return CanPlayUrl(Url, Options, OutWarnings, OutErrors) ? 100 : 0;
	}

	/**
	 * Whether the player works on the given platform.
	 *
	 * @param PlatformName The name of the platform to check.
	 * @return true if the platform is supported, false otherwise.
	 * @see GetSupportedPlatforms
	 */
	bool SupportsPlatform(const FString& PlatformName) const
	{
		return GetSupportedPlatforms().Contains(PlatformName);
	}

public:

	/** Virtual destructor. */
	virtual ~IMediaPlayerFactory() = default;
};
