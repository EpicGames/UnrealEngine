// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/Range.h"
#include "Misc/Timespan.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "IMediaTimeSource.h"

class IMediaAudioSample;
class IMediaBinarySample;
class IMediaOverlaySample;
class IMediaTextureSample;


/**
 * Interface for access to a media player's sample queue.
 *
 * @see IMediaCache, IMediaControls, IMediaPlayer, IMediaTracks, IMediaView
 */
class IMediaSamples
{
public:

	//~ The following methods are optional

	/**
	 * Fetch the next audio sample.
	 *
	 * @param TimeRange The range of present times that the sample is allowed to have.
	 * @param OutSample Will contain the sample if the queue is not empty.
	 * @return true if the returned sample is valid, false otherwise.
	 * @see FetchCaption, FetchMetadata, FetchSubtitle, FetchVideo
	 */
	virtual bool FetchAudio(TRange<FTimespan> TimeRange, TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& OutSample)
	{
		return false; // override in child classes, if supported
	}
	virtual bool FetchAudio(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& OutSample)
	{
		return false; // override in child classes, if supported
	}

	/**
	 * Fetch the next caption sample.
	 *
	 * @param TimeRange The range of present times that the sample is allowed to have.
	 * @param OutSample Will contain the sample if the queue is not empty.
	 * @return true if the returned sample is valid, false otherwise.
	 * @see FetchAudio, FetchMetadata, FetchSubtitle, FetchVideo
	 */
	virtual bool FetchCaption(TRange<FTimespan> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample)
	{
		return false; // override in child classes, if supported
	}
	virtual bool FetchCaption(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample)
	{
		return false; // override in child classes, if supported
	}

	/**
	 * Fetch the next metadata sample.
	 *
	 * @param TimeRange The range of present times that the sample is allowed to have.
	 * @param OutSample Will contain the sample if the queue is not empty.
	 * @return true if the returned sample is valid, false otherwise.
	 * @see FetchAudio, FetchCaption, FetchSubtitle, FetchVideo
	 */
	virtual bool FetchMetadata(TRange<FTimespan> TimeRange, TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe>& OutSample)
	{
		return false; // override in child classes, if supported
	}
	virtual bool FetchMetadata(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe>& OutSample)
	{
		return false; // override in child classes, if supported
	}

	/**
	 * Fetch the next subtitle sample.
	 *
	 * @param TimeRange The range of present times that the sample is allowed to have.
	 * @param OutSample Will contain the sample if the queue is not empty.
	 * @return true if the returned sample is valid, false otherwise.
	 * @see FetchAudio, FetchCaption, FetchMetadata, FetchVideo
	 */
	virtual bool FetchSubtitle(TRange<FTimespan> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample)
	{
		return false; // override in child classes, if supported
	}
	virtual bool FetchSubtitle(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample)
	{
		return false; // override in child classes, if supported
	}

	/**
	 * Fetch the next video sample.
	 *
	 * @param TimeRange The range of present times that the sample is allowed to have.
	 * @param OutSample Will contain the sample if the queue is not empty.
	 * @return true if the returned sample is valid, false otherwise.
	 * @see FetchAudio, FetchCaption, FetchMetadata, FetchSubtitle
	 */
	virtual bool FetchVideo(TRange<FTimespan> TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample)
	{
		return false; // override in child classes, if supported
	}
	virtual bool FetchVideo(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample)
	{
		return false; // override in child classes, if supported
	}

	/** Discard any outstanding media samples. */
	virtual void FlushSamples()
	{
		// override in child classes, if supported
	}

	/** Sets the number of samples to be stored in sample container. */
	virtual void SetSampleBufferSize(int32 BufferSize) {};

	enum class EFetchBestSampleResult
	{
		Ok = 0,
		NoSample,
		PurgedToEmpty,
		NotSupported,
	};
	virtual EFetchBestSampleResult FetchBestVideoSampleForTimeRange(const TRange<FMediaTimeStamp>& TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample, bool bReverse, bool bConsistentResult)
	{
		return EFetchBestSampleResult::NotSupported;
	}
	virtual void SetMinExpectedNextSequenceIndex(TOptional<int32> InNextSequenceIndex) { }

	virtual bool PeekVideoSampleTime(FMediaTimeStamp& TimeStamp) = 0;

	virtual bool PeekVideoSampleTimeRanges(TArray<TRange<FMediaTimeStamp>>& TimeRange) { return false; }
	virtual bool PeekAudioSampleTimeRanges(TArray<TRange<FMediaTimeStamp>>& TimeRange) { return false; }

	virtual bool DiscardVideoSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) { return false; }
	virtual bool DiscardAudioSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) { return false; }
	virtual bool DiscardCaptionSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) { return false; }
	virtual bool DiscardSubtitleSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) { return false; }
	virtual bool DiscardMetadataSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) { return false; }

	virtual uint32 PurgeOutdatedVideoSamples(const FMediaTimeStamp& ReferenceTime, bool bReversed, FTimespan MaxAge) { return 0; };
	virtual uint32 PurgeOutdatedCaptionSamples(const FMediaTimeStamp& ReferenceTime, bool bReversed, FTimespan MaxAge) { return 0; };
	virtual uint32 PurgeOutdatedSubtitleSamples(const FMediaTimeStamp& ReferenceTime, bool bReversed, FTimespan MaxAge) { return 0; };
	virtual uint32 PurgeOutdatedMetadataSamples(const FMediaTimeStamp& ReferenceTime, bool bReversed, FTimespan MaxAge) { return 0; };

	virtual bool CanReceiveVideoSamples(uint32 Num) const { return true; }
	virtual bool CanReceiveAudioSamples(uint32 Num) const { return true; }
	virtual bool CanReceiveSubtitleSamples(uint32 Num) const { return true; }
	virtual bool CanReceiveCaptionSamples(uint32 Num) const { return true; }
	virtual bool CanReceiveMetadataSamples(uint32 Num) const { return true; }

	virtual int32 NumAudioSamples() const { return -1; }
	virtual int32 NumCaptionSamples() const { return -1; }
	virtual int32 NumMetadataSamples() const { return -1; }
	virtual int32 NumSubtitleSamples() const { return -1; }
	virtual int32 NumVideoSamples() const { return -1; }

	virtual uint32 GetNumDroppedVideoSamples(bool bInClearToZero) { return 0U; }
	virtual uint32 GetNumDroppedAudioSamples(bool bInClearToZero) { return 0U; }
	virtual uint32 GetNumDroppedSubtitleSamples(bool bInClearToZero) { return 0U; }
	virtual uint32 GetNumDroppedCaptionSamples(bool bInClearToZero) { return 0U; }
	virtual uint32 GetNumDroppedMetadataSamples(bool bInClearToZero) { return 0U; }

public:

	/** Virtual destructor. */
	virtual ~IMediaSamples() = default;
};
