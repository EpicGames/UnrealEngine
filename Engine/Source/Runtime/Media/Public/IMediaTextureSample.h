// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IMediaTimeSource.h"
#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Math/MathFwd.h"
#include "Math/Matrix.h"
#include "Math/Plane.h"
#include "Misc/Optional.h"
#include "Misc/Timecode.h"
#include "Misc/Timespan.h"
#include "Misc/FrameRate.h"
#include "Templates/SharedPointer.h"

#include "HDRHelper.h"
#include "ColorManagement/ColorManagementDefines.h"
#include "ColorManagement/ColorSpace.h"
#include "MediaShaders.h"

#if WITH_ENGINE
	#include "RHIFwd.h"
	class FRHITexture;
	class IMediaTextureSampleConverter;
	class IMediaTextureSampleColorConverter;
#endif


/**
 * Available formats for media texture samples.
 *
 * Depending on the decoder, the pixel data of a media texture frame may be stored
 * in one of the listed formats. Some of these may be supported natively by the
 * renderer, others may require a conversion step to a natively supported format.
 * The media texture sink is responsible for performing any necessary conversions.
 *
 * For details on the various YUV packings see: https://www.fourcc.org/yuv.php
 */
enum class EMediaTextureSampleFormat
{
	/** Format not defined. */
	Undefined,

	/** Four 8-bit unsigned integer components (AYUV packing) per texel. */
	CharAYUV,

	/** Four 8-bit unsigned integer components (Blue, Green, Red, Alpha) per texel. */
	CharBGRA,

	/** Four 8-bit unsigned integer components (Blue, Green, Red, Alpha) per texel. */
	CharRGBA,

	/** Four 10-bit unsigned integer components (Blue, Green, Red) & 2-bit alpha per texel. */
	CharBGR10A2,

	/**  Windows bitmap (like CharBGRA, but flipped vertically). */
	CharBMP,

	/** NV12 encoded monochrome texture with 8 bits per channel. */
	CharNV12,

	/** NV21 encoded monochrome texture with 8 bits per channel. */
	CharNV21,

	/** Four 8-bit unsigned integer components (UYVY packing aka. HDYC, IUYV, UYNV, Y422) per texel. */
	CharUYVY,

	/** Four 8-bit unsigned integer components (YUY2 packing aka. YUNV, YUYV) per texel. */
	CharYUY2,

	/** Four 8-bit unsigned integer components (UYVY) per texel. */
	Char2VUY,

	/** Four 8-bit unsigned integer components (YVYU packing) per texel. */
	CharYVYU,

	/** Three 16-bit floating point components (Red, Green, Blue) per texel. */
	FloatRGB,

	/** Four 16-bit floating point components (Red, Green, Blue, Alpha) per texel. */
	FloatRGBA,

	/** YUV v210 format which pack 6 pixel using 12 x 10bits components (128 bits block). */
	YUVv210,

	/** YUV v216 format which pack 2 pixel using 4 x 16bits components */
	YUVv216,

	/** Packed YUYV (Y0CbY1Cr) on 16-bit unsigned integer components. */
	ShortYUY2,

	/** 4:4:4:4 AY'CbCr 16-bit little endian full range alpha, video range Y'CbCr. */
	Y416,

	/** 4:4:4:4 AY'CbCr 32-bit little endian full range alpha, video range Y'CbCr. */
	R4FL,

	/** NV12-style encoded monochrome texture with 16 bits per channel, with the upper 10 bits used. */
	P010,

	/** DXT1. */
	DXT1,

	/** DXT5. */
	DXT5,

	/** BC4. */
	BC4,

	/** YCoCg colour space encoded in DXT5. */
	YCoCg_DXT5,

	/** YCoCg colour space encoded in DXT5, with a separate alpha texture encoded in BC4. */
	YCoCg_DXT5_Alpha_BC4,

	/** 3 planes of RGB1010102 data representing Y, U & V at 4:2:0 sampling. */
	P010_RGB1010102,

	/** RGBA 16-bit per component */
	RGBA16,

	/** ABGR 16-bit per component */
	ABGR16,

	/** ARGB 16-bit per component, big endian */
	ARGB16_BIG,

	/** External texture with 4 components (VYUX) per texel. */
	ExternalVYU,
};

namespace MediaTextureSampleFormat
{
	 MEDIA_API const TCHAR* EnumToString(const EMediaTextureSampleFormat InSampleFormat);

	 static inline bool IsBlockCompressedFormat(EMediaTextureSampleFormat InSampleFormat)
	 {
		PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
		switch (InSampleFormat)
		{
		case EMediaTextureSampleFormat::DXT1:
		case EMediaTextureSampleFormat::DXT5:
		case EMediaTextureSampleFormat::BC4:
		case EMediaTextureSampleFormat::YCoCg_DXT5:
		case EMediaTextureSampleFormat::YCoCg_DXT5_Alpha_BC4:
		 return true;
		}
		PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
		return false;
	 }
};

/** Description of how the media texture sample is tiled (only used by tiled image sequences currently).*/
struct FMediaTextureTilingDescription
{
	FIntPoint TileNum = FIntPoint::ZeroValue;
	FIntPoint TileSize = FIntPoint::ZeroValue;
	int32 TileBorderSize = 0;

	inline bool IsValid() const
	{
		return TileNum.X > 0 && TileNum.Y > 0 && TileSize.X > 0 && TileSize.Y > 0;
	}
};

enum class EMediaOrientation
{
	Original = 0,
	CW90,
	CW180,
	CW270
};

namespace MediaTextureSample
{
	/* UE's historical reference white level for mapping scene-referred colors to nits (see TonemapCommon.ush). */
	UE_DEPRECATED(5.8, "Use UE::Color::GetReferenceWhiteLinearScale for pipeline normalization; any additional white-level adjustment should happen in the consuming material.")
	static constexpr float kLinearToNitsScale_UE = 100.0f;

	/* This the reference white level for mapping SDR 1.0 to nits, as defined by ITU-R Report BT.2408. */
	UE_DEPRECATED(5.8, "Use UE::Color::GetReferenceWhiteLinearScale(EEncoding, EReferenceWhite::BT2408) for pipeline normalization; any additional white-level adjustment should happen in the consuming material.")
	static constexpr float kLinearToNitsScale_BT2408 = 203.0f;
}

UE_DEPRECATED(5.8, "Use UE::Color::GetReferenceWhiteLinearScale(EEncoding, EReferenceWhite) for pipeline normalization; any additional white-level adjustment should happen in the consuming material.")
PRAGMA_DISABLE_DEPRECATION_WARNINGS
static constexpr float kMediaSample_HDR_NitsNormalizationFactor = 1.0f / MediaTextureSample::kLinearToNitsScale_UE;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/**
 * Interface for media texture samples.
 *
 * Media texture samples are generated by media players and sent to the registered
 * media texture sink. They contain a single frame of texture data along with extra
 * metadata, such as dimensions, time codes, and durations.
 *
 * Depending on the decoder, a frame's pixel data may be stored in a CPU memory
 * buffer, or in an RHI texture resource (only available when compiling against
 * the Engine). The media texture sample API supports both models via the GetBuffer
 * and the GetTexture methods. Sample instances need to implement only one of these.
 */
class IMediaTextureSample
{
public:
	/**
	 * Get the sample's frame buffer.
	 *
	 * The returned buffer is only valid for the life time of this sample.
	 *
	 * @return Buffer containing the texels, or nullptr if the sample holds an FTexture.
	 * @see GetDim, GetDuration, GetFormat, GetOutputDim, GetStride, GetTexture, GetTime
	 */
	virtual const void* GetBuffer() = 0;

	/**
	 * Get the width and height of the sample.
	 *
	 * The sample may be larger than the output dimensions, because
	 * of horizontal or vertical padding required by some formats.
	 *
	 * @return Buffer dimensions (in texels).
	 * @see GetBuffer, GetDuration, GetFormat, GetOutputDim, GetStride, GetTexture, GetTime
	 */
	virtual FIntPoint GetDim() const = 0;

	/**
	 * Get the number of mips encoded in the sample
	 *
	 * @return Number of mips in the sample (including base level)
	 * @note Default implementation provided as most samples will not feature mips
	 */
	virtual uint8 GetNumMips() const
	{
		return 1;
	}

	/**
	 * Get tile information (number, size and border size) of the sample.
	 *
	 * @return TileInfo struct
	 * @note Default implementation provided as most samples will not feature tiles
	 */
	virtual FMediaTextureTilingDescription GetTilingDescription() const
	{
		return FMediaTextureTilingDescription();
	}

	/**
	 * Get the amount of time for which the sample is valid.
	 *
	 * A duration of zero indicates that the sample is valid until the
	 * timecode of the next sample in the queue.
	 *
	 * @return Sample duration.
	 * @see GetBuffer, GetDim, GetFormat, GetOutputDim, GetStride, GetTexture, GetTime
	 */
	virtual FTimespan GetDuration() const = 0;

	/**
	 * Get the texture sample format.
	 *
	 * @return Sample format.
	 * @see GetBuffer, GetDim, GetDuration, GetOutputDim, GetStride, GetTexture, GetTime
	 */
	virtual EMediaTextureSampleFormat GetFormat() const = 0;

	/**
	 * Get the sample's desired output width and height.
	 *
	 * The output dimensions may be smaller than the frame buffer dimensions, because
	 * of horizontal and/or vertical padding that may be required for some formats.
	 *
	 * @return Output dimensions (in pixels).
	 * @see GetBuffer, GetDim, GetDuration, GetFormat, GetStride, GetTexture, GetTime
	 */
	virtual FIntPoint GetOutputDim() const = 0;

	/**
	 * Get the horizontal stride (aka. pitch) of the sample's frame buffer.
	 *
	 * @return The buffer stride (in number of bytes).
	 * @see GetBuffer, GetDim, GetDuration, GetFormat, GetOutputDim, GetTexture, GetTime
	 */
	virtual uint32 GetStride() const = 0;

	/**
	 * Whether samples should be converted based on a mismatch with the working color space.
	 * If this is false, samples will not go through the conversion process even if their encoding or chromaticity doesn't match the working color space.
	 */
	virtual bool ShouldApplyColorConversion() const { return true; }

#if WITH_ENGINE

	/**
	 * Get the sample's texture resource.
	 *
	 * @return Texture resource, or nullptr if the sample holds a frame buffer.
	 * @see GetBuffer, GetDim, GetDuration, GetFormat, GetOutputDim, GetStride, GetTime
	 */
	virtual FRHITexture* GetTexture() const = 0;

	/**
	 * Get media texture sample converter if sample implements it
	 *
	 * @return texture sample converter
	 */
	virtual IMediaTextureSampleConverter* GetMediaTextureSampleConverter()
	{
		return nullptr;
	}

	/**
	 * Get a media texture sample color converter if sample implements it
	 * @Note IMediaTextureSampleColorConverter will be applied after IMediaTextureSampleConverter if one is provided.
	 * @return texture sample color converter
	 */
	virtual IMediaTextureSampleColorConverter* GetMediaTextureSampleColorConverter()
	{
		return nullptr;
	}

	/**
	 * Extra ETextureCreateFlags to OR into the intermediate render target for this
	 * sample. Lets samples opt into create-time policies (e.g. ReservedResource for
	 * sparse residency) without an RHI-wide engine change. Default: no extra flags.
	 */
	virtual ETextureCreateFlags GetIntermediateTargetCreateFlagsHint() const
	{
		return static_cast<ETextureCreateFlags>(0);
	}

#endif //WITH_ENGINE

	/**
	 * Get the sample time (in the player's local clock).
	 *
	 * This value is used primarily for debugging purposes.
	 *
	 * @return Sample time.
	 * @see GetBuffer, GetDim, GetDuration, GetFormat, GetOutputDim, GetStride, GetTexture
	 */
	virtual FMediaTimeStamp GetTime() const = 0;

	/**
	 * Get the sample timecode if available.
	 *
	 * @return Sample timecode.
	 * @see GetTime
	 */
	virtual TOptional<FTimecode> GetTimecode() const { return TOptional<FTimecode>(); }

	/**
	 * Get the sample framerate if available. This is the rate in which the timecode
	 * is measured. It is not necessarily the display frame rate.
	 *
	 * @return Sample framerate. May be needed for converting GetTimecode().
	 * @see GetTime
	 */
	virtual TOptional<FFrameRate> GetFramerate() const { return TOptional<FFrameRate>(); }

	/**
	 * Whether the sample can be held in a cache.
	 *
	 * Non-cacheable video samples become invalid when the next sample is available,
	 * and only the latest sample should be kept by video sample consumers.
	 *
	 * @return true if cacheable, false otherwise.
	 */
	virtual bool IsCacheable() const = 0;

	/**
	 * Whether the output of the sample is in sRGB color space.
	 *
	 * @return true if sRGB, false otherwise.
	 */
	virtual bool IsOutputSrgb() const = 0;

	/**
	 * Get image orientation vs. physically returned image data
	 *
	 * @return Image orientation
	 */
	virtual EMediaOrientation GetOrientation() const
	{
		return EMediaOrientation::Original;
	}

	/**
	 * Get pixel aspect ratio
	 *
	 * @return Pixel aspect ratio
	 */
	virtual double GetAspectRatio() const
	{
		FIntPoint OutputDim = GetOutputDim();
		return (double)OutputDim.X / (double)OutputDim.Y;
	}

	/**
	 * Indicate if this sample references an "external image"
	 */
	virtual bool IsExternalImage() const
	{
		return false;
	}
	
	/**
	 * Get the ScaleRotation (2x2 matrix) for the sample.
	 *
	 * @return FLinearColor with xy = row 0 (dotted with U), zw = row 1 (dotted with V)
	 *
	 * @note For use with "external image" style output only. Use GetOrientation() otherwise
	 *
	 */
	virtual FLinearColor GetScaleRotation() const
	{
		return FLinearColor(1.0f, 0.0f, 0.0f, 1.0f);
	}

	/**
	 * Get the Offset applied after ScaleRotation for the sample.
	 *
	 * @return FLinearColor with xy = offset, zw must be zero
	 *
	 * @note For use with "external image" style output only
	 *
	 */
	virtual FLinearColor GetOffset() const
	{
		return FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);
	}

	/**
	 * Get the YUV to RGB conversion matrix.
	 *
	 * Default is equivalent to MediaShaders::YuvToRgbRec709Scaled Matrix. NOTE: previously in UE4 this was YuvToRgbRec601Scaled
	 *
	 * @return Conversion Matrix
	 */
	virtual const FMatrix& GetYUVToRGBMatrix() const
	{
		return MediaShaders::YuvToRgbRec709Scaled;
	}

	/*
	* Get full range color flag
	*/
	virtual bool GetFullRange() const
	{
		return false;
	}

	/**
	* Get complete 4x4 matrix to apply to the sample's pixels to yield RGB data in the sample's gamut
	 *
	 * @return Conversion Matrix
	*/
	virtual FMatrix44f GetSampleToRGBMatrix() const
	{
		FMatrix Pre = FMatrix::Identity;
		FVector Off;
		switch (GetFormat())
		{
			case EMediaTextureSampleFormat::R4FL:		Off = MediaShaders::YUVOffsetFloat; break;
			case EMediaTextureSampleFormat::Y416:
			case EMediaTextureSampleFormat::P010:
			case EMediaTextureSampleFormat::ShortYUY2:
			case EMediaTextureSampleFormat::YUVv216:	Off = MediaShaders::YUVOffset16bits; break;
			case EMediaTextureSampleFormat::YUVv210:	Off = MediaShaders::YUVOffset10bits; break;
			default:									Off = MediaShaders::YUVOffset8bits; break;
		}
		Pre.M[0][3] = -Off.X;
		Pre.M[1][3] = -Off.Y;
		Pre.M[2][3] = -Off.Z;
		return FMatrix44f(MediaShaders::YuvToRgbRec709Scaled * Pre);	// assumes sRGB & video range
	}

	/*
	* Get sample source color space (defaults to the sRGB/Rec709 gamut)
	*/
	virtual const UE::Color::FColorSpace& GetSourceColorSpace() const
	{
		return UE::Color::FColorSpace::GetSRGB();
	}

	UE_DEPRECATED(5.5, "GetGamutToXYZMatrix is deprecated, please use GetSourceColorSpace instead.")
	virtual FMatrix44d GetGamutToXYZMatrix() const
	{
		return FMatrix44d(GamutToXYZMatrix(EDisplayColorGamut::sRGB_D65));
	}

	UE_DEPRECATED(5.5, "GetWhitePoint is deprecated, please use GetSourceColorSpace instead.")
	virtual FVector2d GetWhitePoint() const
	{
		return UE::Color::GetWhitePoint(UE::Color::EWhitePoint::CIE1931_D65);
	}

	UE_DEPRECATED(5.5, "GetDisplayPrimaryRed is deprecated, please use GetMasteringDisplayColorSpace instead.")
	virtual FVector2d GetDisplayPrimaryRed() const
	{
		return FVector2d(0.64, 0.33);
	}

	UE_DEPRECATED(5.5, "GetDisplayPrimaryGreen is deprecated, please use GetMasteringDisplayColorSpace instead.")
	virtual FVector2d GetDisplayPrimaryGreen() const
	{
		return FVector2d(0.30, 0.60);
	}

	UE_DEPRECATED(5.5, "GetDisplayPrimaryBlue is deprecated, please use GetMasteringDisplayColorSpace instead.")
	virtual FVector2d GetDisplayPrimaryBlue() const
	{
		return FVector2d(0.15, 0.06);
	}

	/**
	* Chromatic adaptation method to be used when applying a color space transform (i.e. from source to working color space).
	*/
	virtual UE::Color::EChromaticAdaptationMethod GetChromaticAdapationMethod() const
	{
		return UE::Color::DEFAULT_CHROMATIC_ADAPTATION_METHOD;
	}

	/**
	 * Get EOTF / "Gamma" / encoding type of data
	 */
	virtual UE::Color::EEncoding GetEncodingType() const
	{
		return IsOutputSrgb() ? UE::Color::EEncoding::sRGB : UE::Color::EEncoding::Linear;
	}

	/**
	 * Linear-side scale that consumers may apply to normalize decoded transfer-function
	 * output to UE scene-linear (diffuse white = 1.0). Consumers typically multiply the
	 * color-space conversion matrix by this factor; how and when it is applied is up to
	 * the consumer.
	 *
	 * Subclasses should override this when the source metadata indicates a specific
	 * reference white. The canonical implementation calls
	 * UE::Color::GetReferenceWhiteLinearScale(GetEncodingType(), EReferenceWhite::XXX)
	 * so the scale resolves per encoding (e.g. 1/203 for PQ + BT2408, 1/DecodeHLG(0.75)
	 * for HLG + BT2408). The base default returns 1.0 so projects without tagging see
	 * spec-literal decoded values.
	 */
	virtual float GetHDRNitsNormalizationFactor() const
	{
		return 1.0f;
	}

	/**
	 * Get display mastering luminance information
	 */
	virtual bool GetDisplayMasteringLuminance(float& OutMin, float& OutMax) const
	{
		return false;
	}

	/**
	 * Get display mastering color space
	 */
	virtual TOptional<UE::Color::FColorSpace> GetDisplayMasteringColorSpace() const
	{
		return TOptional<UE::Color::FColorSpace>();
	}

	/**
	 * Get maximum luminance information
	 */
	virtual bool GetMaxLuminanceLevels(uint16& OutCLL, uint16& OutFALL) const
	{
		return false;
	}

	/**
	 * Get an optional tonemapping method, for application on HDR inputs.
	 */
	virtual MediaShaders::EToneMapMethod GetToneMapMethod() const
	{
		return MediaShaders::EToneMapMethod::None;
	}

	/**
	 * Reset sample to empty state
	 */
	virtual void Reset() { }

public:

	/** Virtual destructor. */
	virtual ~IMediaTextureSample() = default;
};
