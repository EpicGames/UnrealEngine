// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Bad includes for this module

#include "RHI.h"
#include "RHIResources.h"
#include "RHICommandList.h"

/**
 * Interface class to implement custom sample conversion
 */
class IMediaTextureSampleConverter
{
public:
	virtual ~IMediaTextureSampleConverter() = default;

	enum EConverterInfoFlags
	{
		ConverterInfoFlags_Default = 0,
		ConverterInfoFlags_WillCreateOutputTexture	= 1 << 0,
		ConverterInfoFlags_PreprocessOnly			= 1 << 1,
		ConverterInfoFlags_NeedUAVOutputTexture		= 1 << 2,
	};

	struct FConversionHints
	{
		uint8 NumMips;
	};

	virtual uint32 GetConverterInfoFlags() const
	{
		return ConverterInfoFlags_Default;
	}

	virtual bool Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints) = 0;

	UE_DEPRECATED(5.5, "The overload of IMediaTextureSampleConverter::Convert which does not take an RHI command list is deprecated.")
	bool Convert(FTextureRHIRef& InDstTexture, const FConversionHints& Hints)
	{
		return Convert(FRHICommandListImmediate::Get(), InDstTexture, Hints);
	}
};

/**
 * Interface class to implement custom sample color conversion
 */
class IMediaTextureSampleColorConverter
{
public:
	virtual ~IMediaTextureSampleColorConverter() = default;

	/**
	 * Apply a color conversion on the input and store the result in the destination texture.
	 * @return true If the color conversion was successfully applied.
	 */
	virtual bool ApplyColorConversion(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InSrcTexture, FTextureRHIRef& InDstTexture) = 0;

	UE_DEPRECATED(5.5, "The overload of IMediaTextureSampleColorConverter::ApplyColorConversion which does not take an RHI command list is deprecated.")
	bool ApplyColorConversion(FTextureRHIRef& InSrcTexture, FTextureRHIRef& InDstTexture)
	{
		return ApplyColorConversion(FRHICommandListImmediate::Get(), InSrcTexture, InDstTexture);
	}
};

