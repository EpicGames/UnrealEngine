// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/**
 * Interface for objects that can be ticked from the high-frequency media thread.
 */
class IMediaTickable
{
public:

	/** Tick the object. */
	virtual void TickTickable() = 0;

public:

	/** Virtual destructor. */
	virtual ~IMediaTickable() = default;
};
