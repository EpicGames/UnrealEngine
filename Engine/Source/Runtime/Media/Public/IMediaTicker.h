// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IMediaTickable;


/**
 * Interface for the high-frequency media ticker.
 */
class IMediaTicker
{
public:

	/**
	 * Add a media tickable object.
	 *
	 * @param Tickable The object to register.
	 * @see UnregisterTickable
	 */
	virtual void AddTickable(const TSharedRef<IMediaTickable, ESPMode::ThreadSafe>& Tickable) = 0;

	/**
	 * Remove a media tickable object.
	 *
	 * @param Tickable The object to register.
	 * @see RegisterTickable
	 */
	virtual void RemoveTickable(const TSharedRef<IMediaTickable, ESPMode::ThreadSafe>& Tickable) = 0;

public:

	/** Virtual destructor. */
	virtual ~IMediaTicker() = default;
};
