// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Timespan.h"
#include "Math/NumericLimits.h"

/*
	Timestamp value for media playback

	- Time:
	  Time value

	- SequenceIndex
	  Sequence that is current for this time value

	Within a single sequence time values will increase or decrease monotonically. A new sequence index is generated on each event that causes the time to no
	longer be monotonic. (e.g. seek or loop)
	A sequence index does not offer any information about the ordering of the time stamps on the timeline. Time values are comparable between all timestamps from a single playback,
	though, one needs to be careful to consider non-monotonic behavior if the sequence indices are not identical.

	Sequence indices can very much offer ordering information as far as playback progression is concerned. Higher indices are also later in playback. (even if time values may be smaller: e.g. looping)

	All comparison operators of this class will operate to indicate ordering relative to playback, not position on the timeline!

*/
class FMediaTimeStamp
{
public:
	FMediaTimeStamp() : Time(FTimespan::MinValue())
	{ }
	explicit FMediaTimeStamp(const FTimespan& InTime) : Time(InTime)
	{ }
	explicit FMediaTimeStamp(const FTimespan& InTime, int64 InSequenceValue)
		: Time(InTime)
		, SequenceIndex(GetSequenceIndex(InSequenceValue))
		, LoopIndex(GetLoopIndex(InSequenceValue))
	{ }
	explicit FMediaTimeStamp(const FTimespan& InTime, int32 InSequenceIndex, int32 InLoopIndex)
		: Time(InTime)
		, SequenceIndex(InSequenceIndex)
		, LoopIndex(InLoopIndex)
	{ }

	void Invalidate()
	{
		Time = FTimespan::MinValue();
	}
	bool IsValid() const
	{
		return Time != FTimespan::MinValue();
	}

	bool operator == (const FMediaTimeStamp& Other) const
	{
		return SequenceIndex == Other.SequenceIndex && LoopIndex == Other.LoopIndex && Time == Other.Time;
	}
	bool operator != (const FMediaTimeStamp& Other) const
	{
		return SequenceIndex != Other.SequenceIndex || LoopIndex != Other.LoopIndex || Time != Other.Time;
	}
	bool operator < (const FMediaTimeStamp& Other) const
	{
		return (GetSequenceIndex() < Other.GetSequenceIndex()) ||
			   (GetSequenceIndex() == Other.GetSequenceIndex() && GetLoopIndex() < Other.GetLoopIndex()) ||
			   (GetSequenceIndex() == Other.GetSequenceIndex() && GetLoopIndex() == Other.GetLoopIndex() && GetTime() < Other.GetTime());
	}
	bool operator <= (const FMediaTimeStamp& Other) const
	{
		return this->operator==(Other) || this->operator<(Other);
	}
	bool operator > (const FMediaTimeStamp& Other) const
	{
		return !this->operator<=(Other);
	}
	bool operator >= (const FMediaTimeStamp & Other) const
	{
		return !this->operator<(Other);
	}

	FMediaTimeStamp& SetTimeAndIndexValue(const FTimespan &InTime, int64 InIndexValue)
	{
		Time = InTime;
		SequenceIndex = GetSequenceIndex(InIndexValue);
		LoopIndex = GetLoopIndex(InIndexValue);
		return *this;
	}
	FMediaTimeStamp& SetTime(const FTimespan &InTime)
	{
		Time = InTime;
		return *this;
	}

	FMediaTimeStamp& SetSequenceIndex(int32 InSequenceIndex)
	{
		SequenceIndex = InSequenceIndex;
		return *this;
	}
	FMediaTimeStamp& SetLoopIndex(int32 InLoopIndex)
	{
		LoopIndex = InLoopIndex;
		return *this;
	}

	const FTimespan& GetTime() const
	{
		return Time;
	}
	int64 GetIndexValue() const
	{
		return (int64)(((uint64)SequenceIndex << 32) | (uint32)LoopIndex);
	}

	FMediaTimeStamp& AdjustLoopIndex(int32 Add)
	{
		LoopIndex += Add;
		return *this;
	}

	int32 GetSequenceIndex() const
	{
		return SequenceIndex;
	}

	int32 GetLoopIndex() const
	{
		return LoopIndex;
	}


	static int64 MakeIndexValue(int32 InSequenceIndex, int32 InLoopIndex)
	{
		return (int64)(((uint64)InSequenceIndex << 32) | (uint64)InLoopIndex);
	}

	static int32 GetSequenceIndex(int64 InSequenceIndex)
	{
		return static_cast<int32>(InSequenceIndex >> 32);
	}

	static int32 GetLoopIndex(int64 InSequenceIndex)
	{
		return static_cast<int32>(((uint64)InSequenceIndex) & 0xffffffff);
	}

	FMediaTimeStamp operator + (const FTimespan& Other) const
	{
		return FMediaTimeStamp(Time + Other, SequenceIndex, LoopIndex);
	}
	FMediaTimeStamp operator - (const FTimespan& Other) const
	{
		return FMediaTimeStamp(Time - Other, SequenceIndex, LoopIndex);
	}

	FMediaTimeStamp operator - (const FMediaTimeStamp& Other) const
	{
		if (Other.SequenceIndex == SequenceIndex && Other.LoopIndex == LoopIndex)
		{
			return FMediaTimeStamp(Time - Other.Time, MAX_int32, MAX_int32);
		}
		else if (Other.SequenceIndex <= SequenceIndex || Other.LoopIndex <= LoopIndex)
		{
			return FMediaTimeStamp(FTimespan::MaxValue(), MAX_int32, MAX_int32);
		}
		else
		{
			return FMediaTimeStamp(FTimespan::MinValue(), MAX_int32, MAX_int32);
		}
	}

	FMediaTimeStamp& operator += (const FTimespan& Other)
	{
		Time += Other;
		return *this;
	}
	FMediaTimeStamp& operator -= (const FTimespan& Other)
	{
		Time -= Other;
		return *this;
	}

	FMediaTimeStamp& operator -= (const FMediaTimeStamp& Other)
	{
		Time -= Other.Time;
		SequenceIndex = MAX_int32;
		LoopIndex = MAX_int32;
		return *this;
	}

	FTimespan Time;
private:
	int32 SequenceIndex = 0;
	int32 LoopIndex = 0;
};

class FMediaTimeStampSample
{
public:
	FMediaTimeStampSample() : SampledAtTime(-1.0) {}
	FMediaTimeStampSample(const FMediaTimeStamp & InTimeStamp, double InSampledAtTime) : TimeStamp(InTimeStamp), SampledAtTime(InSampledAtTime) {}

	void Invalidate() { TimeStamp.Invalidate(); SampledAtTime = -1.0; }
	bool IsValid() const { return TimeStamp.IsValid(); }

	FMediaTimeStamp TimeStamp;
	double SampledAtTime;
};

/**
 * Interface for media clock time sources.
 */
class IMediaTimeSource
{
public:

	/**
	 * Get the current time code.
	 *
	 * @return Time code.
	 */
	virtual FTimespan GetTimecode() = 0;

public:

	/** Virtual destructor. */
	virtual ~IMediaTimeSource() = default;
};
