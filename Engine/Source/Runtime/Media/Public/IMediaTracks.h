// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Math/IntPoint.h"
#include "Math/Range.h"
#include "Math/RangeSet.h"
#include "Misc/Optional.h"
#include "Misc/Timespan.h"

/**
 * Enumerates available media track types.
 *
 * Note: Keep this in sync with EMediaTrackType
 */
enum class EMediaTrackType
{
	/** Audio track. */
	Audio,

	/** Closed caption track. */
	Caption,

	/** Metadata track. */
	Metadata,

	/** Script track. */
	Script,

	/** Subtitle track. */
	Subtitle,

	/** Generic text track. */
	Text,

	/** Video track. */
	Video,

	Num
};


/**
 * Audio track format details.
 */
struct FMediaAudioTrackFormat
{
	/** Number of bits per sample. */
	uint32 BitsPerSample;

	/** Number of audio channels. */
	uint32 NumChannels;

	/** Sample rate (in samples per second). */
	uint32 SampleRate;

	/** Name of the format type. */
	FString TypeName;
};


/**
 * Video track format details.
 */
struct FMediaVideoTrackFormat
{
	/** Width and height of the video (in pixels). */
	FIntPoint Dim;

	/** Active frame rate (in frames per second). */
	float FrameRate;

	/** Supported frame rate range. */
	TRange<float> FrameRates;

	/** Name of the format type. */
	FString TypeName;
};


/**
 * Interface for access to a media player's tracks.
 *
 * @see IMediaCache, IMediaControls, IMediaPlayer, IMediaSamples, IMediaView
 */
class IMediaTracks
{
public:

	/**
	 * Returns the duration of a specific track.
	 *
	 * If a media file contains multiple tracks each track may have different duration and
	 * the overall media duration is typically reported as that from the longest track.
	 *
	 * For some applications it may be required to know the exact track duration for processing,
	 * which is what this method returns if the media container, as well as the media player
	 * used for playback has the ability to provide it.
	 *
	 * @param TrackType The type of track to query.
	 * @param TrackIndex The index of the track to query.
	 * @return The duration of the track if known.
	 */
	virtual TOptional<FTimespan> GetTrackDuration(EMediaTrackType TrackType, int32 TrackIndex) const
	{ return TOptional<FTimespan>(); }

	/**
	 * Get details about the specified audio track format.
	 *
	 * @param TrackIndex The index of the audio track (must be valid).
	 * @param FormatIndex The index of the track's format (must be valid).
	 * @param OutFormat Will contain the format details.
	 * @return true on success, false otherwise.
	 * @see GetVideoTrackFormat
	 */
	virtual bool GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const = 0;

	/**
	 * Get the number of media tracks of the given type.
	 *
	 * @param TrackType The type of media tracks.
	 * @return Number of tracks.
	 * @see GetNumTrackFormats, GetSelectedTrack, SelectTrack
	 */
	virtual int32 GetNumTracks(EMediaTrackType TrackType) const = 0;

	/**
	 * Get the number of formats of the specified track.
	 *
	 * @param TrackType The type of track.
	 * @param TrackIndex The index of the track (must be valid).
	 * @return Number of formats.
	 * @see GetNumTracks, GetTrackFormatType, GetTrackDisplayName
	 */
	virtual int32 GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const = 0;

	/**
	 * Get the index of the currently selected track of the given type.
	 *
	 * @param TrackType The type of track to get.
	 * @return The index of the selected track, or INDEX_NONE if no track is active.
	 * @see GetNumTracks, SelectTrack
	 */
	virtual int32 GetSelectedTrack(EMediaTrackType TrackType) const = 0;

	/**
	 * Get the human readable name of the specified track.
	 *
	 * @param TrackType The type of track.
	 * @param TrackIndex The index of the track (must be valid).
	 * @return Display name.
	 * @see GetTrackLanguage, GetTrackName
	 */
	virtual FText GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const = 0;

	/**
	 * Get the index of the currently selected format of the given track type.
	 *
	 * @param TrackType The type of track.
	 * @param TrackIndex The index of the track (must be valid).
	 * @return The index of the selected format.
	 * @see GetNumTrackFormats, GetSelectedTrack, SetTrackFormat
	 */
	virtual int32 GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const = 0;

	/**
	 * Get the language tag of the specified track.
	 *
	 * @param TrackType The type of track.
	 * @param TrackIndex The index of the track (must be valid).
	 * @return Language tag, i.e. "en-US" for English, or "und" for undefined.
	 * @see GetTrackDisplayName, GetTrackName
	 */
	virtual FString GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const = 0;

	/**
	 * Get the internal name of the specified track.
	 *
	 * @param TrackType The type of track.
	 * @param TrackIndex The index of the track (must be valid).
	 * @return Track name.
	 * @see GetTrackDisplayName, GetTrackLanguage
	 */
	virtual FString GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const = 0;

	/**
	 * Get details about the specified video track format.
	 *
	 * @param TrackIndex The index of the video track (must be valid).
	 * @param FormatIndex The index of the track's format (must be valid).
	 * @param OutFormat Will contain the format details.
	 * @return true on success, false otherwise.
	 * @see GetAudioTrackFormat
	 */
	virtual bool GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const = 0;

	/**
	 * Select the active track of the given type.
	 *
	 * The selected track will use its currently active format. Active formats will
	 * be remembered on a per track basis. The first available format is active by
	 * default. To switch the track format, use SetTrackFormat instead.
	 *
	 * @param TrackType The type of track to select.
	 * @param TrackIndex The index of the track to select, or INDEX_NONE to deselect.
	 * @return true if the track was selected, false otherwise.
	 * @see GetNumTracks, GetSelectedTrack, SetTrackFormat
	 */
	virtual bool SelectTrack(EMediaTrackType TrackType, int32 TrackIndex) = 0;

	/**
	 * Select the active format on the specified track.
	 *
	 * Selecting the format will not switch to the specified track. To switch
	 * tracks, use SelectTrack instead. If the track is already selected, the
	 * format change will be applied immediately.
	 *
	 * @param TrackType The type of track to update.
	 * @param TrackIndex The index of the track to update (must be valid).
	 * @param FormatIndex The index of the format to select (must be valid).
	 * @return true if the format was selected, false otherwise.
	 * @see GetNumTrackFormats, GetNumTracks, GetTrackFormat, SelectTrack
	 */
	virtual bool SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex) = 0;

	/**
	 * Set the frame rate of the specified video track.
	 *
	 * Note that most players may not support overriding the video frame rate.
	 * This feature is often only available on video capture media sources.
	 *
	 * @param TrackIndex The index of the track (must be valid).
	 * @param FormatIndex Index of the track format (must be valid).
	 * @param FrameRate The frame rate to set (must be in range of format's supported frame rates).
	 * @return true on success, false otherwise.
	 * @see GetTrackFormat, SetTrackFormat
	 */
	virtual bool SetVideoTrackFrameRate(int32 TrackIndex, int32 FormatIndex, float FrameRate)
	{
		return false;
	}

public:

	/** Virtual destructor. */
	virtual ~IMediaTracks() = default;
};
