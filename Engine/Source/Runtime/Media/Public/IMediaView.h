// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Quat.h"
#include "Containers/Set.h"
#include "Containers/Map.h"

/**
 * Tile coordinate of a tiled media source/texture.
 */
struct UE_DEPRECATED(5.7, "Unused since virtual texture support was never fully submitted.") FMediaTileCoordinate
{
	uint16 X, Y;

	FMediaTileCoordinate() = default;

	inline explicit FMediaTileCoordinate(uint16 InX, uint16 InY)
		: X(InX)
		, Y(InY)
	{
	}

	inline explicit FMediaTileCoordinate(int32 InX, int32 InY)
		: X(IntCastChecked<uint16>(InX))
		, Y(IntCastChecked<uint16>(InY))
	{
	}

	inline FMediaTileCoordinate(EForceInit)
		: X(0u)
		, Y(0u)
	{
	}

	inline bool operator==(const FMediaTileCoordinate& Other) const
	{
		return X == Other.X && Y == Other.Y;
	}


	inline bool operator!=(const FMediaTileCoordinate& Other) const
	{
		return X != Other.X || Y != Other.Y;
	}

	inline static FMediaTileCoordinate Zero()
	{
		return FMediaTileCoordinate(EForceInit::ForceInit);
	}
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
template <> struct TIsPODType<FMediaTileCoordinate> { enum { Value = true }; };


inline uint32 GetTypeHash(const FMediaTileCoordinate& Coord)
{
	return FCrc::TypeCrc32(Coord);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS


/**
 * Interface for a media player's viewing settings.
 *
 * This interface is used to control viewing parameters in media players that
 * support 360-degree or stereoscopic video output, or spatial audio output.
 * There are currently three sets of configurable parameters: view, view focus
 * and user settings.
 *
 * The view is the area that the user is looking at. Media players may use it
 * to optimize the decoding of the media, i.e. only decode the sub-section of
 * the video that is actually visible.
 *
 * The view focus is an optional area that allows for specifying which area the
 * user is or should be focusing on. Media players may use it for foveated video
 * rendering or for making the focused area more audible.
 *
 * The user settings can be used by media players to customize the generated
 * audio or video output for specific users.
 * 
 * Note: This view class has now been extended to also support tiled
 * media sources such as tiled exr image sequences. Sets of visible tiles can
 * be specified per mip level.
 *
 * @see IMediaCache, IMediaControls, IMediaPlayer, IMediaSamples, IMediaTracks
 */
class IMediaView
{
public:

	//~ View parameters

	/**
	 * Get the field of view.
	 *
	 * @param OutHorizontal Will contain the horizontal field of view.
	 * @param OutVertical Will contain the vertical field of view.
	 * @return true on success, false if feature is not available or if field of view has never been set.
	 * @see GetViewOrientation, SetViewField
	 */
	virtual bool GetViewField(float& OutHorizontal, float& OutVertical) const
	{
		return false;
	}

	/**
	 * Get the view's orientation.
	 *
	 * @param OutOrientation Will contain the view orientation.
	 * @return true on success, false if feature is not available or if orientation has never been set.
	 * @see GetViewField, SetViewOrientation
	 */
	virtual bool GetViewOrientation(FQuat& OutOrientation) const
	{
		return false;
	}

	/**
	 * Set the field of view.
	 *
	 * @param Horizontal Horizontal field of view (in Euler degrees).
	 * @param Vertical Vertical field of view (in Euler degrees).
	 * @param Whether the field of view change should be absolute (true) or relative (false).
	 * @return true on success, false otherwise.
	 * @see GetViewField, SetViewFocusField, SetViewOrientation
	 */
	virtual bool SetViewField(float Horizontal, float Vertical, bool Absolute)
	{
		return false; // override in child classes if supported
	}

	/**
	 * Set the view's orientation.
	 *
	 * @param Orientation Quaternion representing the orientation.
	 * @param Whether the orientation change should be absolute (true) or relative (false).
	 * @return true on success, false otherwise.
	 * @see GetViewOrientation, SetViewField
	 */
	virtual bool SetViewOrientation(const FQuat& Orientation, bool Absolute)
	{
		return false; // override in child classes if supported
	}

public:

	//~ View focus parameters

	/**
	 * Whether the view focus is enabled.
	 *
	 * @return true if enabled, false otherwise.
	 */
	virtual bool IsViewFocusEnabled() const
	{
		return false; // override in child classes if supported
	}

	/**
	 * Enable or disable view focus.
	 *
	 * @param Enabled true to enable, false to disable.
	 * @return true on success, false otherwise.
	 * @see IsViewFocusEnabled, SetViewFocusField, SetViewFocusOffImportance, SetViewFocusOrientation
	 */
	virtual bool SetViewFocusEnabled(bool Enabled)
	{
		return false; // override in child classes if supported
	}

	/**
	 * Set the view's focused field of view.
	 *
	 * @param Horizontal Horizontal field of view (in Euler degrees).
	 * @param Vertical Vertical field of view (in Euler degrees).
	 * @return true on success, false otherwise.
	 * @see SetViewFocusEnabled, SetViewFocusOffImportance, SetViewField, SetViewFocusOrientation
	 */
	virtual bool SetViewFocusField(float Horizontal, float Vertical)
	{
		return false; // override in child classes if supported
	}

	/**
	 * Set the importance of the area that is not in focus.
	 *
	 * @param Importance Off-area importance (0.0 = no importance, 1.0 = important).
	 * @return true on success, false otherwise.
	 * @see SetViewFocusEnabled, SetViewFocusField, SetViewFocusOrientation
	 */
	virtual bool SetViewFocusOffImportance(float Importance)
	{
		return false; // override in child classes if supported
	}

	/**
	 * Set the view's focused orientation.
	 *
	 * @param Orientation Quaternion representing the viewer's orientation.
	 * @return true on success, false otherwise.
	 * @see SetViewFocusEnabled, SetViewFocusOffImportance, SetViewFocusField, SetViewField
	 */
	virtual bool SetViewFocusOrientation(const FQuat& Orientation)
	{
		return false; // override in child classes if supported
	}

public:

	//~ User parameters

	/**
	 * Set the inter-ocular distance.
	 *
	 * @param Distance The inter-ocular distance (in centimeters).
	 * @return true on success, false otherwise.
	 */
	virtual bool SetInteroccularDistance(float Distance)
	{
		return false; // override in child classes if supported
	}

public:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	//~ Visible tile interface

	/**
	 * Get the visible tiles per mip-level.
	 *
	 * @param OutTiles Map of visible tile sets per mip level index.
	 * @return true on success, false if feature is not available.
	 * @see SetVisibleTiles
	 */
	UE_DEPRECATED(5.7, "Unused since virtual texture support was never fully submitted.")
	virtual bool GetVisibleTiles(TMap<int32, TSet<FMediaTileCoordinate>>& OutTiles) const
	{
		return false; // override in child classes if supported
	};

	/**
	 * Set the visible tiles per mip-level.
	 *
	 * @param InTiles Map of visible tile sets per mip level index.
	 * @return true on success, false if feature is not available.
	 * @see GetVisibleTiles
	 */
	UE_DEPRECATED(5.7, "Unused since virtual texture support was never fully submitted.")
	virtual bool SetVisibleTiles(TMap<int32, TSet<FMediaTileCoordinate>>&& InTiles)
	{
		return false; // override in child classes if supported
	};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:

	/** Virtual destructor. */
	virtual ~IMediaView() { }
};
